{
	"FileVersion" : 3,
	"Version" : 14,
	"VersionName" : "1.4",
	"FriendlyName" : "Git",
	"Description" : "Git source control management",
	"Category" : "Source Control",
	"CreatedBy" : "SRombauts and Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "GitSourceControl",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		}
	]
}