// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GitSourceControl : ModuleRules
{
	public GitSourceControl(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;
		
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"Slate",
				"SlateCore",
				"InputCore",
				"DesktopWidgets",
				"SourceControl",
			}
		);

		if (Target.bBuildEditor == true)
		{
			// needed to enable/disable this via experimental settings
			PrivateDependencyModuleNames.Add("CoreUObject");
			PrivateDependencyModuleNames.Add("EditorFramework");
			PrivateDependencyModuleNames.Add("UnrealEd");
		}

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
