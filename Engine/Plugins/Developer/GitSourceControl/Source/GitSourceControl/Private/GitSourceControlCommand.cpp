// Copyright Epic Games, Inc. All Rights Reserved.

#include "GitSourceControlCommand.h"
#include "Modules/ModuleManager.h"
#include "GitSourceControlModule.h"

FGitSourceControlCommand::FGitSourceControlCommand(const TSharedRef<class ISourceControlOperation, ESPMode::ThreadSafe>& InOperation, const TSharedRef<class IGitSourceControlWorker, ESPMode::ThreadSafe>& InWorker, const FSourceControlOperationComplete& InOperationCompleteDelegate)
	: Operation(InOperation)
	, Worker(InWorker)
	, OperationCompleteDelegate(InOperationCompleteDelegate)
	, bExecuteProcessed(0)
	, bCommandSuccessful(false)
	, bAutoDelete(true)
	, Concurrency(EConcurrency::Synchronous)
{
	// grab the providers settings here, so we don't access them once the worker thread is launched
	check(IsInGameThread());
	FGitSourceControlModule& GitSourceControl = FModuleManager::LoadModuleChecked<FGitSourceControlModule>( "GitSourceControl" );
	PathToGitBinary = GitSourceControl.AccessSettings().GetBinaryPath();
	PathToRepositoryRoot = GitSourceControl.GetProvider().GetPathToRepositoryRoot();
}

bool FGitSourceControlCommand::DoWork()
{
	bCommandSuccessful = Worker->Execute(*this);
	FPlatformAtomics::InterlockedExchange(&bExecuteProcessed, 1);

	return bCommandSuccessful;
}

void FGitSourceControlCommand::Abandon()
{
	FPlatformAtomics::InterlockedExchange(&bExecuteProcessed, 1);
}

void FGitSourceControlCommand::DoThreadedWork()
{
	Concurrency = EConcurrency::Asynchronous;
	DoWork();
}

ECommandResult::Type FGitSourceControlCommand::ReturnResults()
{
	// Save any messages that have accumulated
	for (FString& String : InfoMessages)
	{
		Operation->AddInfoMessge(FText::FromString(String));
	}
	for (FString& String : ErrorMessages)
	{
		Operation->AddErrorMessge(FText::FromString(String));
	}

	// run the completion delegate if we have one bound
	ECommandResult::Type Result = bCommandSuccessful ? ECommandResult::Succeeded : ECommandResult::Failed;
	OperationCompleteDelegate.ExecuteIfBound(Operation, Result);

	return Result;
}
