// Copyright Epic Games, Inc. All Rights Reserved.

#include "GitSourceControlModule.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "GitSourceControlOperations.h"
#include "Features/IModularFeatures.h"

#define LOCTEXT_NAMESPACE "GitSourceControl"

template<typename Type>
static TSharedRef<IGitSourceControlWorker, ESPMode::ThreadSafe> CreateWorker()
{
	return MakeShareable( new Type() );
}

void FGitSourceControlModule::StartupModule()
{
	// Register our operations
	GitSourceControlProvider.RegisterWorker( "Connect", FGetGitSourceControlWorker::CreateStatic( &CreateWorker<FGitConnectWorker> ) );
	// Note: this provider does not uses the "CheckOut" command, which is a Perforce "lock", as Git has no lock command (all tracked files in the working copy are always already checked-out).
	GitSourceControlProvider.RegisterWorker( "UpdateStatus", FGetGitSourceControlWorker::CreateStatic( &CreateWorker<FGitUpdateStatusWorker> ) );
	GitSourceControlProvider.RegisterWorker( "MarkForAdd", FGetGitSourceControlWorker::CreateStatic( &CreateWorker<FGitMarkForAddWorker> ) );
	GitSourceControlProvider.RegisterWorker( "Delete", FGetGitSourceControlWorker::CreateStatic( &CreateWorker<FGitDeleteWorker> ) );
	GitSourceControlProvider.RegisterWorker( "Revert", FGetGitSourceControlWorker::CreateStatic( &CreateWorker<FGitRevertWorker> ) );
	GitSourceControlProvider.RegisterWorker( "Sync", FGetGitSourceControlWorker::CreateStatic( &CreateWorker<FGitSyncWorker> ) );
	GitSourceControlProvider.RegisterWorker( "CheckIn", FGetGitSourceControlWorker::CreateStatic( &CreateWorker<FGitCheckInWorker> ) );
	GitSourceControlProvider.RegisterWorker( "Copy", FGetGitSourceControlWorker::CreateStatic( &CreateWorker<FGitCopyWorker> ) );
	GitSourceControlProvider.RegisterWorker( "Resolve", FGetGitSourceControlWorker::CreateStatic( &CreateWorker<FGitResolveWorker> ) );

	// load our settings
	GitSourceControlSettings.LoadSettings();

	// Bind our source control provider to the editor
	IModularFeatures::Get().RegisterModularFeature( "SourceControl", &GitSourceControlProvider );
}

void FGitSourceControlModule::ShutdownModule()
{
	// shut down the provider, as this module is going away
	GitSourceControlProvider.Close();

	// unbind provider from editor
	IModularFeatures::Get().UnregisterModularFeature("SourceControl", &GitSourceControlProvider);
}

void FGitSourceControlModule::SaveSettings()
{
	if (FApp::IsUnattended() || IsRunningCommandlet())
	{
		return;
	}

	GitSourceControlSettings.SaveSettings();
}

IMPLEMENT_MODULE(FGitSourceControlModule, GitSourceControl);

#undef LOCTEXT_NAMESPACE
