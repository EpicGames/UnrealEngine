// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "GitSourceControlSettings.h"
#include "GitSourceControlProvider.h"

/**

UEGitPlugin is a simple Git Source Control Plugin for Unreal Engine

Written and contributed by Sebastien Rombauts (sebastien.rombauts@gmail.com)

### Supported features
- initialize a new Git local repository ('git init') to manage your UE Game Project.
- create an appropriate .gitignore file as part as initialization
- can also make the initial commit
- display status icons to show modified/added/deleted/untracked files
- show history of a file
- visual diff of a blueprint against depot or between previous versions of a file
- revert modifications of a file
- add, delete, rename a file
- checkin/commit a file (cannot handle atomically more than 20 files)
- migrate an asset between to projects if both are using Git
- solve a merge conflict on a blueprint
- show current branch name in status text
- Sync to Pull the current branch if there is no local modified files
- git LFS (Github & Gitlab), git-annex and/or git-media are working under Windows
- Windows, Mac and Linux

### What *cannot* be done presently
- tags: implement ISourceControlLabel to manage git tags
- Branch is not in the current Editor workflow (but on Epic Roadmap)
- Pull/Fetch/Push are not in the current Editor workflow
- Amend a commit is not in the current Editor workflow
- configure user name & email ('git config user.name' & git config user.email')

### Known issues:
- the Editor does not show deleted files (only when deleted externaly?)
- the Editor does not show missing files
- the Editor does not show .uproject file state
- missing localisation for git specific messages
- displaying states of 'Engine' assets (also needs management of 'out of tree' files)
- issue #22: A Move/Rename leaves a redirector file behind
- reverting an asset does not seem to update content in Editor! Issue in Editor?
- renaming a Blueprint in Editor leaves a tracker file, AND modify too much the asset to enable git to track its history through renaming
- file history show Changelist as signed integer instead of hexadecimal SHA1
- standard Editor commit dialog ask if user wants to "Keep Files Checked Out" => no use for Git or Mercurial CanCheckOut()==false

 */
class FGitSourceControlModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** Access the Git source control settings */
	FGitSourceControlSettings& AccessSettings()
	{
		return GitSourceControlSettings;
	}

	/** Save the Git source control settings */
	void SaveSettings();

	/** Access the Git source control provider */
	FGitSourceControlProvider& GetProvider()
	{
		return GitSourceControlProvider;
	}

private:
	/** The Git source control provider */
	FGitSourceControlProvider GitSourceControlProvider;

	/** The settings for Git source control */
	FGitSourceControlSettings GitSourceControlSettings;
};
