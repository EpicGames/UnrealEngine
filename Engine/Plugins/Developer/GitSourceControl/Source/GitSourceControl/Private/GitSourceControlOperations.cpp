// Copyright Epic Games, Inc. All Rights Reserved.

#include "GitSourceControlOperations.h"
#include "Misc/Paths.h"
#include "SourceControlOperations.h"
#include "ISourceControlModule.h"
#include "GitSourceControlModule.h"
#include "GitSourceControlCommand.h"
#include "GitSourceControlUtils.h"

#define LOCTEXT_NAMESPACE "GitSourceControl"

FName FGitConnectWorker::GetName() const
{
	return "Connect";
}

bool FGitConnectWorker::Execute(FGitSourceControlCommand& InCommand)
{
	check(InCommand.Operation->GetName() == GetName());

	// Check Git Availability
	if (0 < InCommand.PathToGitBinary.Len() && GitSourceControlUtils::CheckGitAvailability(InCommand.PathToGitBinary))
	{
		// Now update the status of assets in Content/ directory and also Config files
		TArray<FString> ProjectDirs;
		ProjectDirs.Add(FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir()));
		ProjectDirs.Add(FPaths::ConvertRelativePathToFull(FPaths::ProjectConfigDir()));
		InCommand.bCommandSuccessful = GitSourceControlUtils::RunUpdateStatus(InCommand.PathToGitBinary, InCommand.PathToRepositoryRoot, ProjectDirs, InCommand.ErrorMessages, States);
		if(!InCommand.bCommandSuccessful || InCommand.ErrorMessages.Num() > 0)
		{
			StaticCastSharedRef<FConnect>(InCommand.Operation)->SetErrorText(LOCTEXT("NotAGitRepository", "Failed to enable Git revision control. You need to initialize the project as a Git repository first."));
			InCommand.bCommandSuccessful = false;
		}
	}
	else
	{
		StaticCastSharedRef<FConnect>(InCommand.Operation)->SetErrorText(LOCTEXT("GitNotFound", "Failed to enable Git revision control. You need to install Git and specify a valid path to git executable."));
		InCommand.bCommandSuccessful = false;
	}

	return InCommand.bCommandSuccessful;
}

bool FGitConnectWorker::UpdateStates() const
{
	return GitSourceControlUtils::UpdateCachedStates(States);
}

static FText ParseCommitResults(const TArray<FString>& InResults)
{
	if(InResults.Num() >= 1)
	{
		const FString& FirstLine = InResults[0];
		return FText::Format(LOCTEXT("CommitMessage", "Commited {0}."), FText::FromString(FirstLine));
	}
	return LOCTEXT("CommitMessageUnknown", "Submitted revision.");
}

FName FGitCheckInWorker::GetName() const
{
	return "CheckIn";
}

bool FGitCheckInWorker::Execute(FGitSourceControlCommand& InCommand)
{
	check(InCommand.Operation->GetName() == GetName());

	TSharedRef<FCheckIn, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FCheckIn>(InCommand.Operation);

	// make a temp file to place our commit message in
	FGitScopedTempFile CommitMsgFile(Operation->GetDescription());
	if(CommitMsgFile.GetFilename().Len() > 0)
	{
		TArray<FString> Parameters;
		FString ParamCommitMsgFilename = TEXT("--file=\"");
		ParamCommitMsgFilename += FPaths::ConvertRelativePathToFull(CommitMsgFile.GetFilename());
		ParamCommitMsgFilename += TEXT("\"");
		Parameters.Add(ParamCommitMsgFilename);

		InCommand.bCommandSuccessful = GitSourceControlUtils::RunCommit(InCommand.PathToGitBinary, InCommand.PathToRepositoryRoot, Parameters, InCommand.Files, InCommand.InfoMessages, InCommand.ErrorMessages);
		if(InCommand.bCommandSuccessful)
		{
			// Remove any deleted files from status cache
			FGitSourceControlModule& GitSourceControl = FModuleManager::GetModuleChecked<FGitSourceControlModule>("GitSourceControl");
			FGitSourceControlProvider& Provider = GitSourceControl.GetProvider();

			TArray<TSharedRef<ISourceControlState, ESPMode::ThreadSafe>> LocalStates;
			Provider.GetState(InCommand.Files, LocalStates, EStateCacheUsage::Use);
			for (const auto& State : LocalStates)
			{
				if (State->IsDeleted())
				{
					Provider.RemoveFileFromCache(State->GetFilename());
				}
			}

			Operation->SetSuccessMessage(ParseCommitResults(InCommand.InfoMessages));
			const FString Message = (InCommand.InfoMessages.Num() > 0) ? InCommand.InfoMessages[0] : TEXT("");
			UE_LOGF(LogSourceControl, Log, "commit successful: %ls", *Message);
		}
	}

	// now update the status of our files
	GitSourceControlUtils::RunUpdateStatus(InCommand.PathToGitBinary, InCommand.PathToRepositoryRoot, InCommand.Files, InCommand.ErrorMessages, States);

	return InCommand.bCommandSuccessful;
}

bool FGitCheckInWorker::UpdateStates() const
{
	return GitSourceControlUtils::UpdateCachedStates(States);
}

FName FGitMarkForAddWorker::GetName() const
{
	return "MarkForAdd";
}

bool FGitMarkForAddWorker::Execute(FGitSourceControlCommand& InCommand)
{
	check(InCommand.Operation->GetName() == GetName());

	InCommand.bCommandSuccessful = GitSourceControlUtils::RunCommand(TEXT("add"), InCommand.PathToGitBinary, InCommand.PathToRepositoryRoot, TArray<FString>(), InCommand.Files, InCommand.InfoMessages, InCommand.ErrorMessages);

	// now update the status of our files
	GitSourceControlUtils::RunUpdateStatus(InCommand.PathToGitBinary, InCommand.PathToRepositoryRoot, InCommand.Files, InCommand.ErrorMessages, States);

	return InCommand.bCommandSuccessful;
}

bool FGitMarkForAddWorker::UpdateStates() const
{
	return GitSourceControlUtils::UpdateCachedStates(States);
}

FName FGitDeleteWorker::GetName() const
{
	return "Delete";
}

bool FGitDeleteWorker::Execute(FGitSourceControlCommand& InCommand)
{
	check(InCommand.Operation->GetName() == GetName());

	InCommand.bCommandSuccessful = GitSourceControlUtils::RunCommand(TEXT("rm"), InCommand.PathToGitBinary, InCommand.PathToRepositoryRoot, TArray<FString>(), InCommand.Files, InCommand.InfoMessages, InCommand.ErrorMessages);

	// now update the status of our files
	GitSourceControlUtils::RunUpdateStatus(InCommand.PathToGitBinary, InCommand.PathToRepositoryRoot, InCommand.Files, InCommand.ErrorMessages, States);

	return InCommand.bCommandSuccessful;
}

bool FGitDeleteWorker::UpdateStates() const
{
	return GitSourceControlUtils::UpdateCachedStates(States);
}


// Get lists of Missing files (ie "deleted"), Existing files, and "other than Added" Existing files
void GetMissingVsExistingFiles(const TArray<FString>& InFiles, TArray<FString>& OutMissingFiles, TArray<FString>& OutAllExistingFiles, TArray<FString>& OutOtherThanAddedExistingFiles)
{
	FGitSourceControlModule& GitSourceControl = FModuleManager::GetModuleChecked<FGitSourceControlModule>("GitSourceControl");
	FGitSourceControlProvider& Provider = GitSourceControl.GetProvider();

	TArray<TSharedRef<ISourceControlState, ESPMode::ThreadSafe>> LocalStates;
	Provider.GetState(InFiles, LocalStates, EStateCacheUsage::Use);
	for(const auto& State : LocalStates)
	{
		if(FPaths::FileExists(State->GetFilename()))
		{
			if(State->IsAdded())
			{
				OutAllExistingFiles.Add(State->GetFilename());
			}
			else
			{
				OutOtherThanAddedExistingFiles.Add(State->GetFilename());
				OutAllExistingFiles.Add(State->GetFilename());
			}
		}
		else
		{
			OutMissingFiles.Add(State->GetFilename());
		}
	}
}

FName FGitRevertWorker::GetName() const
{
	return "Revert";
}

bool FGitRevertWorker::Execute(FGitSourceControlCommand& InCommand)
{
	// Filter files by status to use the right "revert" commands on them
	TArray<FString> MissingFiles;
	TArray<FString> AllExistingFiles;
	TArray<FString> OtherThanAddedExistingFiles;
	GetMissingVsExistingFiles(InCommand.Files, MissingFiles, AllExistingFiles, OtherThanAddedExistingFiles);

	if(MissingFiles.Num() > 0)
	{
		// "Added" files that have been deleted needs to be removed from source control
		InCommand.bCommandSuccessful = GitSourceControlUtils::RunCommand(TEXT("rm"), InCommand.PathToGitBinary, InCommand.PathToRepositoryRoot, TArray<FString>(), MissingFiles, InCommand.InfoMessages, InCommand.ErrorMessages);
	}
	if(AllExistingFiles.Num() > 0)
	{
		// reset any changes already added to the index
		InCommand.bCommandSuccessful = GitSourceControlUtils::RunCommand(TEXT("reset"), InCommand.PathToGitBinary, InCommand.PathToRepositoryRoot, TArray<FString>(), AllExistingFiles, InCommand.InfoMessages, InCommand.ErrorMessages);
	}
	if(OtherThanAddedExistingFiles.Num() > 0)
	{
		// revert any changes in working copy (this would fails if the asset was in "Added" state, since after "reset" it is now "untracked")
		InCommand.bCommandSuccessful = GitSourceControlUtils::RunCommand(TEXT("checkout"), InCommand.PathToGitBinary, InCommand.PathToRepositoryRoot, TArray<FString>(), OtherThanAddedExistingFiles, InCommand.InfoMessages, InCommand.ErrorMessages);
	}

	// now update the status of our files
	GitSourceControlUtils::RunUpdateStatus(InCommand.PathToGitBinary, InCommand.PathToRepositoryRoot, InCommand.Files, InCommand.ErrorMessages, States);

	return InCommand.bCommandSuccessful;
}

bool FGitRevertWorker::UpdateStates() const
{
	return GitSourceControlUtils::UpdateCachedStates(States);
}

FName FGitSyncWorker::GetName() const
{
   return "Sync";
}

bool FGitSyncWorker::Execute(FGitSourceControlCommand& InCommand)
{
   // pull the branch to get remote changes by rebasing any local commits (not merging them to avoid complex graphs)
   // (this cannot work if any local files are modified but not commited)
   {
      TArray<FString> Parameters;
      Parameters.Add(TEXT("--rebase origin HEAD"));
      InCommand.bCommandSuccessful = GitSourceControlUtils::RunCommand(TEXT("pull"), InCommand.PathToGitBinary, InCommand.PathToRepositoryRoot, Parameters, TArray<FString>(), InCommand.InfoMessages, InCommand.ErrorMessages);
   }

   // now update the status of our files
   GitSourceControlUtils::RunUpdateStatus(InCommand.PathToGitBinary, InCommand.PathToRepositoryRoot, InCommand.Files, InCommand.ErrorMessages, States);

   return InCommand.bCommandSuccessful;
}

bool FGitSyncWorker::UpdateStates() const
{
   return GitSourceControlUtils::UpdateCachedStates(States);
}

FName FGitUpdateStatusWorker::GetName() const
{
	return "UpdateStatus";
}

bool FGitUpdateStatusWorker::Execute(FGitSourceControlCommand& InCommand)
{
	check(InCommand.Operation->GetName() == GetName());

	TSharedRef<FUpdateStatus, ESPMode::ThreadSafe> Operation = StaticCastSharedRef<FUpdateStatus>(InCommand.Operation);

	if(InCommand.Files.Num() > 0)
	{
		InCommand.bCommandSuccessful = GitSourceControlUtils::RunUpdateStatus(InCommand.PathToGitBinary, InCommand.PathToRepositoryRoot, InCommand.Files, InCommand.ErrorMessages, States);
		GitSourceControlUtils::RemoveRedundantErrors(InCommand, TEXT("' is outside repository"));

		if(Operation->ShouldUpdateHistory())
		{
			for(int32 Index = 0; Index < States.Num(); Index++)
			{
				FString& File = InCommand.Files[Index];
				TGitSourceControlHistory History;

				if(States[Index].IsConflicted())
				{
					// In case of a merge conflict, we first need to get the tip of the "remote branch" (MERGE_HEAD)
					GitSourceControlUtils::RunGetHistory(InCommand.PathToGitBinary, InCommand.PathToRepositoryRoot, File, true, InCommand.ErrorMessages, History);
				}
				// Get the history of the file in the current branch
				InCommand.bCommandSuccessful &= GitSourceControlUtils::RunGetHistory(InCommand.PathToGitBinary, InCommand.PathToRepositoryRoot, File, false, InCommand.ErrorMessages, History);
				Histories.Add(*File, History);
			}
		}
	}
	else
	{
		// Perforce "opened files" are those that have been modified (or added/deleted): that is what we get with a simple Git status from the root
		if(Operation->ShouldGetOpenedOnly())
		{
			TArray<FString> Files;
			Files.Add(FPaths::ConvertRelativePathToFull(FPaths::ProjectDir()));
			InCommand.bCommandSuccessful = GitSourceControlUtils::RunUpdateStatus(InCommand.PathToGitBinary, InCommand.PathToRepositoryRoot, Files, InCommand.ErrorMessages, States);
		}
	}

	// don't use the ShouldUpdateModifiedState() hint here as it is specific to Perforce: the above normal Git status has already told us this information (like Git and Mercurial)

	return InCommand.bCommandSuccessful;
}

bool FGitUpdateStatusWorker::UpdateStates() const
{
	bool bUpdated = GitSourceControlUtils::UpdateCachedStates(States);

	FGitSourceControlModule& GitSourceControl = FModuleManager::LoadModuleChecked<FGitSourceControlModule>( "GitSourceControl" );
	FGitSourceControlProvider& Provider = GitSourceControl.GetProvider();

	// add history, if any
	for(const auto& History : Histories)
	{
		TSharedRef<FGitSourceControlState, ESPMode::ThreadSafe> State = Provider.GetStateInternal(History.Key);
		State->History = History.Value;
		State->TimeStamp = FDateTime::Now();
		bUpdated = true;
	}

	return bUpdated;
}

FName FGitCopyWorker::GetName() const
{
	return "Copy";
}

bool FGitCopyWorker::Execute(FGitSourceControlCommand& InCommand)
{
	check(InCommand.Operation->GetName() == GetName());

	// Copy or Move operation on a single file : Git does not need an explicit copy nor move,
	// but after a Move the Editor create a redirector file with the old asset name that points to the new asset.
	// The redirector needs to be commited with the new asset to perform a real rename.
	// => the following is to "MarkForAdd" the redirector, but it still need to be committed by selecting the whole directory and "check-in"
	InCommand.bCommandSuccessful = GitSourceControlUtils::RunCommand(TEXT("add"), InCommand.PathToGitBinary, InCommand.PathToRepositoryRoot, TArray<FString>(), InCommand.Files, InCommand.InfoMessages, InCommand.ErrorMessages);

	return InCommand.bCommandSuccessful;
}

bool FGitCopyWorker::UpdateStates() const
{
	return GitSourceControlUtils::UpdateCachedStates(OutStates);
}

FName FGitResolveWorker::GetName() const
{
	return "Resolve";
}

bool FGitResolveWorker::Execute( class FGitSourceControlCommand& InCommand )
{
	check(InCommand.Operation->GetName() == GetName());

	// mark the conflicting files as resolved:
	{
		TArray<FString> Results;
		InCommand.bCommandSuccessful = GitSourceControlUtils::RunCommand(TEXT("add"), InCommand.PathToGitBinary, InCommand.PathToRepositoryRoot, TArray<FString>(), InCommand.Files, Results, InCommand.ErrorMessages);
	}

	// now update the status of our files
	GitSourceControlUtils::RunUpdateStatus(InCommand.PathToGitBinary, InCommand.PathToRepositoryRoot, InCommand.Files, InCommand.ErrorMessages, States);

	return InCommand.bCommandSuccessful;
}

bool FGitResolveWorker::UpdateStates() const
{
	return GitSourceControlUtils::UpdateCachedStates(States);
}

#undef LOCTEXT_NAMESPACE
