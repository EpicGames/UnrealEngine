// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IGitSourceControlWorker.h"
#include "GitSourceControlState.h"

/** Called when first activated on a project, and then at project load time.
 *  Look for the root directory of the git repository (where the ".git/" subdirectory is located). */
class FGitConnectWorker : public IGitSourceControlWorker
{
public:
	virtual ~FGitConnectWorker() {}
	// IGitSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FGitSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

public:
	/** Temporary states for results */
	TArray<FGitSourceControlState> States;
};

/** Commit (check-in) a set of file to the local depot. */
class FGitCheckInWorker : public IGitSourceControlWorker
{
public:
	virtual ~FGitCheckInWorker() {}
	// IGitSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FGitSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

public:
	/** Temporary states for results */
	TArray<FGitSourceControlState> States;
};

/** Add an untraked file to source control (so only a subset of the git add command). */
class FGitMarkForAddWorker : public IGitSourceControlWorker
{
public:
	virtual ~FGitMarkForAddWorker() {}
	// IGitSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FGitSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

public:
	/** Temporary states for results */
	TArray<FGitSourceControlState> States;
};

/** Delete a file and remove it from source control. */
class FGitDeleteWorker : public IGitSourceControlWorker
{
public:
	virtual ~FGitDeleteWorker() {}
	// IGitSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FGitSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

public:
	/** Map of filenames to Git state */
	TArray<FGitSourceControlState> States;
};

/** Revert any change to a file to its state on the local depot. */
class FGitRevertWorker : public IGitSourceControlWorker
{
public:
	virtual ~FGitRevertWorker() {}
	// IGitSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FGitSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

public:
	/** Map of filenames to Git state */
	TArray<FGitSourceControlState> States;
};

/** Git pull --rebase to update branch from its configure remote */
class FGitSyncWorker : public IGitSourceControlWorker
{
public:
	virtual ~FGitSyncWorker() {}
	// IGitSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FGitSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

public:
	/// Map of filenames to Git state
	TArray<FGitSourceControlState> States;
};

/** Get source control status of files on local working copy. */
class FGitUpdateStatusWorker : public IGitSourceControlWorker
{
public:
	virtual ~FGitUpdateStatusWorker() {}
	// IGitSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FGitSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

public:
	/** Temporary states for results */
	TArray<FGitSourceControlState> States;

	/** Map of filenames to history */
	TMap<FString, TGitSourceControlHistory> Histories;
};

/** Copy or Move operation on a single file */
class FGitCopyWorker : public IGitSourceControlWorker
{
public:
	virtual ~FGitCopyWorker() {}
	// IGitSourceControlWorker interface
	virtual FName GetName() const override;
	virtual bool Execute(class FGitSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;

public:
	/** Temporary states for results */
	TArray<FGitSourceControlState> OutStates;
};

/** git add to mark a conflict as resolved */
class FGitResolveWorker : public IGitSourceControlWorker
{
public:
	virtual ~FGitResolveWorker() {}
	virtual FName GetName() const override;
	virtual bool Execute(class FGitSourceControlCommand& InCommand) override;
	virtual bool UpdateStates() const override;
	
private:
	/** Temporary states for results */
	TArray<FGitSourceControlState> States;
};
