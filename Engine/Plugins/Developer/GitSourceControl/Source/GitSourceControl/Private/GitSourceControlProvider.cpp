// Copyright Epic Games, Inc. All Rights Reserved.

#include "GitSourceControlProvider.h"
#include "GitSourceControlState.h"
#include "Misc/Paths.h"
#include "Misc/QueuedThreadPool.h"
#include "GitSourceControlCommand.h"
#include "ISourceControlModule.h"
#include "SourceControlHelpers.h"
#include "GitSourceControlModule.h"
#include "GitSourceControlUtils.h"
#include "SGitSourceControlSettings.h"
#include "SourceControlOperations.h"
#include "Logging/MessageLog.h"
#include "ScopedSourceControlProgress.h"

#define LOCTEXT_NAMESPACE "GitSourceControl"

static FName ProviderName("Git");

void FGitSourceControlProvider::Init(bool bForceConnection)
{
	// Init() is called multiple times at startup: do not check git each time
	if(!bGitAvailable)
	{
		CheckGitAvailability();
	}

	// bForceConnection: not used anymore
}

void FGitSourceControlProvider::CheckGitAvailability()
{
	FGitSourceControlModule& GitSourceControl = FModuleManager::LoadModuleChecked<FGitSourceControlModule>("GitSourceControl");
	FString PathToGitBinary = GitSourceControl.AccessSettings().GetBinaryPath();
	if(PathToGitBinary.IsEmpty())
	{
		// Try to find Git binary, and update settings accordingly
		PathToGitBinary = GitSourceControlUtils::FindGitBinaryPath();
		if(!PathToGitBinary.IsEmpty())
		{
			GitSourceControl.AccessSettings().SetBinaryPath(PathToGitBinary);
		}
	}

	if(!PathToGitBinary.IsEmpty())
	{
		bGitAvailable = GitSourceControlUtils::CheckGitAvailability(PathToGitBinary, &GitVersion);
		if(bGitAvailable)
		{
			CheckRepositoryStatus(PathToGitBinary);
		}
	}
	else
	{
		bGitAvailable = false;
	}
}

void FGitSourceControlProvider::CheckRepositoryStatus(const FString& InPathToGitBinary)
{
	// Find the path to the root Git directory (if any)
	const FString PathToProjectDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());
	bGitRepositoryFound = GitSourceControlUtils::FindRootDirectory(PathToProjectDir, PathToRepositoryRoot);
	if(bGitRepositoryFound)
	{
		// Get branch name
		bGitRepositoryFound = GitSourceControlUtils::GetBranchName(InPathToGitBinary, PathToRepositoryRoot, BranchName);
		if(bGitRepositoryFound)
		{
			GitSourceControlUtils::GetRemoteUrl(InPathToGitBinary, PathToRepositoryRoot, RemoteUrl);
		}
		else
		{
			UE_LOGF(LogSourceControl, Error, "'%ls' is not a valid Git repository", *PathToRepositoryRoot);
		}
	}
	else
	{
		UE_LOGF(LogSourceControl, Warning, "'%ls' is not part of a Git repository", *FPaths::ProjectDir());
	}

	// Get user name & email (of the repository, else from the global Git config)
	GitSourceControlUtils::GetUserConfig(InPathToGitBinary, PathToRepositoryRoot, UserName, UserEmail);
}

void FGitSourceControlProvider::Close()
{
	// clear the cache
	StateCache.Empty();

	bGitAvailable = false;
	bGitRepositoryFound = false;
	UserName.Empty();
	UserEmail.Empty();
}

TSharedRef<FGitSourceControlState, ESPMode::ThreadSafe> FGitSourceControlProvider::GetStateInternal(const FString& Filename)
{
	TSharedRef<FGitSourceControlState, ESPMode::ThreadSafe>* State = StateCache.Find(Filename);
	if(State != NULL)
	{
		// found cached item
		return (*State);
	}
	else
	{
		// cache an unknown state for this item
		TSharedRef<FGitSourceControlState, ESPMode::ThreadSafe> NewState = MakeShareable( new FGitSourceControlState(Filename) );
		StateCache.Add(Filename, NewState);
		return NewState;
	}
}

FText FGitSourceControlProvider::GetStatusText() const
{
	FFormatNamedArguments Args;
	Args.Add( TEXT("RepositoryName"), FText::FromString(PathToRepositoryRoot) );
	Args.Add( TEXT("RemoteUrl"), FText::FromString(RemoteUrl) );
	Args.Add( TEXT("BranchName"), FText::FromString(BranchName) );
	Args.Add( TEXT("UserName"), FText::FromString(UserName) );
	Args.Add( TEXT("UserEmail"), FText::FromString(UserEmail) );

	return FText::Format( NSLOCTEXT("Status", "Provider: Git\nEnabledLabel", "Local repository: {RepositoryName}\nRemote origin: {RemoteUrl}\nBranch: {BranchName}\nUser: {UserName}\nE-mail: {UserEmail}"), Args );
}

TMap<ISourceControlProvider::EStatus, FString> FGitSourceControlProvider::GetStatus() const
{
	TMap<EStatus, FString> Result;
	Result.Add(EStatus::Enabled, IsEnabled() ? TEXT("Yes") : TEXT("No") );
	Result.Add(EStatus::Connected, (IsEnabled() && IsAvailable()) ? TEXT("Yes") : TEXT("No") );
	Result.Add(EStatus::User, UserName);
	Result.Add(EStatus::Repository, PathToRepositoryRoot);
	Result.Add(EStatus::Remote, RemoteUrl);
	Result.Add(EStatus::Branch, BranchName);
	Result.Add(EStatus::Email, UserEmail);
	return Result;
}

/** Quick check if source control is enabled */
bool FGitSourceControlProvider::IsEnabled() const
{
	return bGitRepositoryFound;
}

/** Quick check if source control is available for use (useful for server-based providers) */
bool FGitSourceControlProvider::IsAvailable() const
{
	return bGitRepositoryFound;
}

const FName& FGitSourceControlProvider::GetName(void) const
{
	return ProviderName;
}

ECommandResult::Type FGitSourceControlProvider::GetState( const TArray<FString>& InFiles, TArray< TSharedRef<ISourceControlState, ESPMode::ThreadSafe> >& OutState, EStateCacheUsage::Type InStateCacheUsage )
{
	if(!IsEnabled())
	{
		return ECommandResult::Failed;
	}

	TArray<FString> AbsoluteFiles = SourceControlHelpers::AbsoluteFilenames(InFiles);

	if(InStateCacheUsage == EStateCacheUsage::ForceUpdate)
	{
		Execute(ISourceControlOperation::Create<FUpdateStatus>(), AbsoluteFiles);
	}

	for(const auto& AbsoluteFile : AbsoluteFiles)
	{
		OutState.Add(GetStateInternal(*AbsoluteFile));
	}

	return ECommandResult::Succeeded;
}

ECommandResult::Type FGitSourceControlProvider::GetState(const TArray<FSourceControlChangelistRef>& InChangelists, TArray<FSourceControlChangelistStateRef>& OutState, EStateCacheUsage::Type InStateCacheUsage)
{
	return ECommandResult::Failed;
}

TArray<FSourceControlStateRef> FGitSourceControlProvider::GetCachedStateByPredicate(TFunctionRef<bool(const FSourceControlStateRef&)> Predicate) const
{
	TArray<FSourceControlStateRef> Result;
	for(const auto& CacheItem : StateCache)
	{
		FSourceControlStateRef State = CacheItem.Value;
		if(Predicate(State))
		{
			Result.Add(State);
		}
	}
	return Result;
}

bool FGitSourceControlProvider::RemoveFileFromCache(const FString& Filename)
{
	return StateCache.Remove(Filename) > 0;
}

FDelegateHandle FGitSourceControlProvider::RegisterSourceControlStateChanged_Handle( const FSourceControlStateChanged::FDelegate& SourceControlStateChanged )
{
	return OnSourceControlStateChanged.Add( SourceControlStateChanged );
}

void FGitSourceControlProvider::UnregisterSourceControlStateChanged_Handle( FDelegateHandle Handle )
{
	OnSourceControlStateChanged.Remove( Handle );
}

ECommandResult::Type FGitSourceControlProvider::Execute( const FSourceControlOperationRef& InOperation, FSourceControlChangelistPtr InChangelist, const TArray<FString>& InFiles, EConcurrency::Type InConcurrency, const FSourceControlOperationComplete& InOperationCompleteDelegate )
{
	if(!IsEnabled() && !(InOperation->GetName() == "Connect")) // Only Connect operation allowed while not Enabled (Connected)
	{
		// Note that IsEnabled() always returns true so unless it is changed, this code will never be executed
		InOperationCompleteDelegate.ExecuteIfBound(InOperation, ECommandResult::Failed);
		return ECommandResult::Failed;
	}

	TArray<FString> AbsoluteFiles = SourceControlHelpers::AbsoluteFilenames(InFiles);

	// Query to see if we allow this operation
	TSharedPtr<IGitSourceControlWorker, ESPMode::ThreadSafe> Worker = CreateWorker(InOperation->GetName());
	if(!Worker.IsValid())
	{
		// this operation is unsupported by this source control provider
		FFormatNamedArguments Arguments;
		Arguments.Add( TEXT("OperationName"), FText::FromName(InOperation->GetName()) );
		Arguments.Add( TEXT("ProviderName"), FText::FromName(GetName()) );
		FText Message(FText::Format(LOCTEXT("UnsupportedOperation", "Operation '{OperationName}' not supported by revision control provider '{ProviderName}'"), Arguments));
		FMessageLog("SourceControl").Error(Message);
		InOperation->AddErrorMessge(Message);

		InOperationCompleteDelegate.ExecuteIfBound(InOperation, ECommandResult::Failed);
		return ECommandResult::Failed;
	}

	FGitSourceControlCommand* Command = new FGitSourceControlCommand(InOperation, Worker.ToSharedRef());
	Command->Files = AbsoluteFiles;
	Command->OperationCompleteDelegate = InOperationCompleteDelegate;

	// fire off operation
	if(InConcurrency == EConcurrency::Synchronous)
	{
		Command->bAutoDelete = false;
		return ExecuteSynchronousCommand(*Command, InOperation->GetInProgressString());
	}
	else
	{
		Command->bAutoDelete = true;
		return IssueCommand(*Command);
	}
}

bool FGitSourceControlProvider::CanExecuteOperation( const FSourceControlOperationRef& InOperation ) const
{
	return WorkersMap.Find(InOperation->GetName()) != nullptr;
}

bool FGitSourceControlProvider::CanCancelOperation( const FSourceControlOperationRef& InOperation ) const
{
	return false;
}

void FGitSourceControlProvider::CancelOperation( const FSourceControlOperationRef& InOperation )
{
}

bool FGitSourceControlProvider::UsesLocalReadOnlyState() const
{
	return false;
}

bool FGitSourceControlProvider::UsesChangelists() const
{
	return false;
}

bool FGitSourceControlProvider::UsesUncontrolledChangelists() const
{
	return true;
}

bool FGitSourceControlProvider::UsesCheckout() const
{
	return false;
}

bool FGitSourceControlProvider::UsesFileRevisions() const
{
	return false;
}

bool FGitSourceControlProvider::UsesSnapshots() const
{
	return false;
}

bool FGitSourceControlProvider::UsesSoftRevertOnDelete() const
{
	return false;
}

bool FGitSourceControlProvider::AllowsDiffAgainstDepot() const
{
	return true;
}

TOptional<bool> FGitSourceControlProvider::HasChangesToSync() const
{
	return TOptional<bool>();
}

TOptional<bool> FGitSourceControlProvider::HasChangesToCheckIn() const
{
	return TOptional<bool>();
}

TSharedPtr<IGitSourceControlWorker, ESPMode::ThreadSafe> FGitSourceControlProvider::CreateWorker(const FName& InOperationName) const
{
	const FGetGitSourceControlWorker* Operation = WorkersMap.Find(InOperationName);
	if(Operation != nullptr)
	{
		return Operation->Execute();
	}

	return nullptr;
}

void FGitSourceControlProvider::RegisterWorker( const FName& InName, const FGetGitSourceControlWorker& InDelegate )
{
	WorkersMap.Add( InName, InDelegate );
}

void FGitSourceControlProvider::OutputCommandMessages(const FGitSourceControlCommand& InCommand) const
{
	FMessageLog SourceControlLog("SourceControl");

	for(int32 ErrorIndex = 0; ErrorIndex < InCommand.ErrorMessages.Num(); ++ErrorIndex)
	{
		SourceControlLog.Error(FText::FromString(InCommand.ErrorMessages[ErrorIndex]));
	}

	for(int32 InfoIndex = 0; InfoIndex < InCommand.InfoMessages.Num(); ++InfoIndex)
	{
		SourceControlLog.Info(FText::FromString(InCommand.InfoMessages[InfoIndex]));
	}
}

void FGitSourceControlProvider::Tick()
{
	bool bStatesUpdated = false;
	for(int32 CommandIndex = 0; CommandIndex < CommandQueue.Num(); ++CommandIndex)
	{
		FGitSourceControlCommand& Command = *CommandQueue[CommandIndex];
		if(Command.bExecuteProcessed)
		{
			// Remove command from the queue
			CommandQueue.RemoveAt(CommandIndex);

			// let command update the states of any files
			bStatesUpdated |= Command.Worker->UpdateStates();

			// dump any messages to output log
			OutputCommandMessages(Command);

			Command.ReturnResults();

			// commands that are left in the array during a tick need to be deleted
			if(Command.bAutoDelete)
			{
				// Only delete commands that are not running 'synchronously'
				delete &Command;
			}

			// only do one command per tick loop, as we dont want concurrent modification
			// of the command queue (which can happen in the completion delegate)
			break;
		}
	}

	if(bStatesUpdated)
	{
		OnSourceControlStateChanged.Broadcast();
	}
}

TArray< TSharedRef<ISourceControlLabel> > FGitSourceControlProvider::GetLabels( const FString& InMatchingSpec ) const
{
	TArray< TSharedRef<ISourceControlLabel> > Tags;

	return Tags;
}

TArray<FSourceControlChangelistRef> FGitSourceControlProvider::GetChangelists( EStateCacheUsage::Type InStateCacheUsage )
{
	return TArray<FSourceControlChangelistRef>();
}

#if SOURCE_CONTROL_WITH_SLATE
TSharedRef<class SWidget> FGitSourceControlProvider::MakeSettingsWidget() const
{
	return SNew(SGitSourceControlSettings);
}
#endif

ECommandResult::Type FGitSourceControlProvider::ExecuteSynchronousCommand(FGitSourceControlCommand& InCommand, const FText& Task)
{
	ECommandResult::Type Result = ECommandResult::Failed;

	// Display the progress dialog if a string was provided
	{
		FScopedSourceControlProgress Progress(Task);

		// Issue the command asynchronously...
		IssueCommand( InCommand );

		// ... then wait for its completion (thus making it synchronous)
		while(!InCommand.bExecuteProcessed)
		{
			// Tick the command queue and update progress.
			Tick();

			Progress.Tick();

			// Sleep for a bit so we don't busy-wait so much.
			FPlatformProcess::Sleep(0.01f);
		}

		// always do one more Tick() to make sure the command queue is cleaned up.
		Tick();

		if(InCommand.bCommandSuccessful)
		{
			Result = ECommandResult::Succeeded;
		}
	}

	// Delete the command now (asynchronous commands are deleted in the Tick() method)
	check(!InCommand.bAutoDelete);

	// ensure commands that are not auto deleted do not end up in the command queue
	if ( CommandQueue.Contains( &InCommand ) )
	{
		CommandQueue.Remove( &InCommand );
	}
	delete &InCommand;

	return Result;
}

ECommandResult::Type FGitSourceControlProvider::IssueCommand(FGitSourceControlCommand& InCommand)
{
	if(GThreadPool != nullptr)
	{
		// Queue this to our worker thread(s) for resolving
		GThreadPool->AddQueuedWork(&InCommand);
		CommandQueue.Add(&InCommand);
		return ECommandResult::Succeeded;
	}
	else
	{
		FText Message(LOCTEXT("NoSCCThreads", "There are no threads available to process the revision control command."));

		FMessageLog("SourceControl").Error(Message);
		InCommand.bCommandSuccessful = false;
		InCommand.Operation->AddErrorMessge(Message);

		return InCommand.ReturnResults();
	}
}
#undef LOCTEXT_NAMESPACE
