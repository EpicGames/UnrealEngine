// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISourceControlProvider.h"
#include "IGitSourceControlWorker.h"

class FGitSourceControlState;

class FGitSourceControlCommand;

DECLARE_DELEGATE_RetVal(FGitSourceControlWorkerRef, FGetGitSourceControlWorker)

struct FGitVersion
{
	int Major;
	int Minor;

	uint32 bHasCatFileWithFilters : 1;
	uint32 bHasGitLfs : 1;
	uint32 bHasGitLfsLocking : 1;

	FGitVersion() 
		: Major(0)
		, Minor(0)
		, bHasCatFileWithFilters(false)
		, bHasGitLfs(false)
		, bHasGitLfsLocking(false)
	{
	}

	inline bool IsGreaterOrEqualThan(int InMajor, int InMinor) const
	{
		return (Major > InMajor) || (Major == InMajor && (Minor >= InMinor));
	}
};

class FGitSourceControlProvider : public ISourceControlProvider
{
public:
	/** Constructor */
	FGitSourceControlProvider() 
		: bGitAvailable(false)
		, bGitRepositoryFound(false)
	{
	}

	/* ISourceControlProvider implementation */
	virtual void Init(bool bForceConnection = true) override;
	virtual void Close() override;
	virtual FText GetStatusText() const override;
	virtual TMap<EStatus, FString> GetStatus() const override;
	virtual bool IsEnabled() const override;
	virtual bool IsAvailable() const override;
	virtual const FName& GetName(void) const override;
	virtual bool QueryStateBranchConfig(const FString& ConfigSrc, const FString& ConfigDest) override { return false;  }
	virtual void RegisterStateBranches(const TArray<FString>& BranchNames, const FString& ContentRoot) override {}
	virtual int32 GetStateBranchIndex(const FString& InBranchName) const override { return INDEX_NONE; }
	virtual bool GetStateBranchAtIndex(int32 BranchIndex, FString& OutBranchName) const override { return false; }
	virtual ECommandResult::Type GetState( const TArray<FString>& InFiles, TArray<FSourceControlStateRef>& OutState, EStateCacheUsage::Type InStateCacheUsage ) override;
	virtual ECommandResult::Type GetState(const TArray<FSourceControlChangelistRef>& InChangelists, TArray<FSourceControlChangelistStateRef>& OutState, EStateCacheUsage::Type InStateCacheUsage) override;
	virtual TArray<FSourceControlStateRef> GetCachedStateByPredicate(TFunctionRef<bool(const FSourceControlStateRef&)> Predicate) const override;
	virtual FDelegateHandle RegisterSourceControlStateChanged_Handle(const FSourceControlStateChanged::FDelegate& SourceControlStateChanged) override;
	virtual void UnregisterSourceControlStateChanged_Handle(FDelegateHandle Handle) override;
	virtual ECommandResult::Type Execute(const FSourceControlOperationRef& InOperation, FSourceControlChangelistPtr InChangelist, const TArray<FString>& InFiles, EConcurrency::Type InConcurrency = EConcurrency::Synchronous, const FSourceControlOperationComplete& InOperationCompleteDelegate = FSourceControlOperationComplete()) override;
	virtual bool CanExecuteOperation( const FSourceControlOperationRef& InOperation ) const override;
	virtual bool CanCancelOperation( const FSourceControlOperationRef& InOperation ) const override;
	virtual void CancelOperation( const FSourceControlOperationRef& InOperation ) override;
	virtual bool UsesLocalReadOnlyState() const override;
	virtual bool UsesChangelists() const override;
	virtual bool UsesUncontrolledChangelists() const override;
	virtual bool UsesCheckout() const override;
	virtual bool UsesFileRevisions() const override;
	virtual bool UsesSnapshots() const override;
	virtual bool UsesSoftRevertOnDelete() const override;
	virtual bool AllowsDiffAgainstDepot() const override;
	virtual TOptional<bool> HasChangesToSync() const override;
	virtual TOptional<bool> HasChangesToCheckIn() const override;
	virtual void Tick() override;
	virtual TArray< TSharedRef<class ISourceControlLabel> > GetLabels( const FString& InMatchingSpec ) const override;
	virtual TArray<FSourceControlChangelistRef> GetChangelists( EStateCacheUsage::Type InStateCacheUsage ) override;
#if SOURCE_CONTROL_WITH_SLATE
	virtual TSharedRef<class SWidget> MakeSettingsWidget() const override;
#endif

	using ISourceControlProvider::Execute;

	/**
	 * Check configuration, else standard paths, and run a Git "version" command to check the availability of the binary.
	 */
	void CheckGitAvailability();

	/**
	 * Find the .git/ repository and check it's status.
	 */
	void CheckRepositoryStatus(const FString& InPathToGitBinary);

	/** Is git binary found and working. */
	inline bool IsGitAvailable() const
	{
		return bGitAvailable;
	}

	/** Git version for feature checking */
	inline const FGitVersion& GetGitVersion()
	{
		return GitVersion;
	}

	/** Get the path to the root of the Git repository: can be the ProjectDir itself, or any parent directory */
	inline const FString& GetPathToRepositoryRoot() const
	{
		return PathToRepositoryRoot;
	}

	/** Git config user.name */
	inline const FString& GetUserName() const
	{
		return UserName;
	}

	/** Git config user.email */
	inline const FString& GetUserEmail() const
	{
		return UserEmail;
	}

	/** Git remote origin url */
	inline const FString& GetRemoteUrl() const
	{
		return RemoteUrl;
	}

	/** Helper function used to update state cache */
	TSharedRef<FGitSourceControlState, ESPMode::ThreadSafe> GetStateInternal(const FString& Filename);

	/**
	 * Register a worker with the provider.
	 * This is used internally so the provider can maintain a map of all available operations.
	 */
	void RegisterWorker( const FName& InName, const FGetGitSourceControlWorker& InDelegate );

	/** Remove a named file from the state cache */
	bool RemoveFileFromCache(const FString& Filename);

private:

	/** Is git binary found and working. */
	bool bGitAvailable;

	/** Is git repository found. */
	bool bGitRepositoryFound;

	/** Helper function for Execute() */
	TSharedPtr<class IGitSourceControlWorker, ESPMode::ThreadSafe> CreateWorker(const FName& InOperationName) const;

	/** Helper function for running command synchronously. */
	ECommandResult::Type ExecuteSynchronousCommand(class FGitSourceControlCommand& InCommand, const FText& Task);
	/** Issue a command asynchronously if possible. */
	ECommandResult::Type IssueCommand(class FGitSourceControlCommand& InCommand);

	/** Output any messages this command holds */
	void OutputCommandMessages(const class FGitSourceControlCommand& InCommand) const;

	/** Path to the root of the Git repository: can be the ProjectDir itself, or any parent directory (found by the "Connect" operation) */
	FString PathToRepositoryRoot;

	/** Git config user.name (from local repository, else globally) */
	FString UserName;

	/** Git config user.email (from local repository, else globally) */
	FString UserEmail;

	/** Name of the current branch */
	FString BranchName;

	/** URL of the "origin" defaut remote server */
	FString RemoteUrl;

	/** State cache */
	TMap<FString, TSharedRef<class FGitSourceControlState, ESPMode::ThreadSafe> > StateCache;

	/** The currently registered source control operations */
	TMap<FName, FGetGitSourceControlWorker> WorkersMap;

	/** Queue for commands given by the main thread */
	TArray < FGitSourceControlCommand* > CommandQueue;

	/** For notifying when the source control states in the cache have changed */
	FSourceControlStateChanged OnSourceControlStateChanged;

	/** Git version for feature checking */
	FGitVersion GitVersion;
};
