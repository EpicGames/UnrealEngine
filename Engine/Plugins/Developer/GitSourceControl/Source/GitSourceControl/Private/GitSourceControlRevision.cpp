// Copyright Epic Games, Inc. All Rights Reserved.

#include "GitSourceControlRevision.h"

#include "ISourceControlModule.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "GitSourceControlModule.h"
#include "GitSourceControlUtils.h"

#define LOCTEXT_NAMESPACE "GitSourceControl"

bool FGitSourceControlRevision::Get(FString& InOutFilename, EConcurrency::Type InConcurrency) const
{
	if (InConcurrency != EConcurrency::Synchronous)
	{
		UE_LOGF(LogSourceControl, Warning, "Only EConcurrency::Synchronous is tested/supported for this operation.");
	}

	FGitSourceControlModule& GitSourceControl = FModuleManager::LoadModuleChecked<FGitSourceControlModule>("GitSourceControl");
	const FString PathToGitBinary = GitSourceControl.AccessSettings().GetBinaryPath();
	const FString PathToRepositoryRoot = GitSourceControl.GetProvider().GetPathToRepositoryRoot();

	// if a filename for the temp file wasn't supplied generate a unique-ish one
	if(InOutFilename.Len() == 0)
	{
		// create the diff dir if we don't already have it (Git wont)
		IFileManager::Get().MakeDirectory(*FPaths::DiffDir(), true);
		// create a unique temp file name based on the unique commit Id
		const FString TempFileName = FString::Printf(TEXT("%stemp-%s-%s"), *FPaths::DiffDir(), *CommitId, *FPaths::GetCleanFilename(Filename));
		InOutFilename = FPaths::ConvertRelativePathToFull(TempFileName);
	}

	// Diff against the revision
	const FString Parameter = FString::Printf(TEXT("%s:%s"), *CommitId, *Filename);

	bool bCommandSuccessful;
	if(FPaths::FileExists(InOutFilename))
	{
		bCommandSuccessful = true; // if the temp file already exists, reuse it directly
	}
	else
	{
		bCommandSuccessful = GitSourceControlUtils::RunDumpToFile(PathToGitBinary, PathToRepositoryRoot, Parameter, InOutFilename);
	}
	return bCommandSuccessful;
}

bool FGitSourceControlRevision::GetAnnotated( TArray<FAnnotationLine>& OutLines ) const
{
	return false;
}

bool FGitSourceControlRevision::GetAnnotated( FString& InOutFilename ) const
{
	return false;
}

const FString& FGitSourceControlRevision::GetFilename() const
{
	return Filename;
}

int32 FGitSourceControlRevision::GetRevisionNumber() const
{
	return RevisionNumber;
}

const FString& FGitSourceControlRevision::GetRevision() const
{
	return ShortCommitId;
}

const FString& FGitSourceControlRevision::GetDescription() const
{
	return Description;
}

const FString& FGitSourceControlRevision::GetUserName() const
{
	return UserName;
}

const FString& FGitSourceControlRevision::GetClientSpec() const
{
	static FString EmptyString(TEXT(""));
	return EmptyString;
}

const FString& FGitSourceControlRevision::GetAction() const
{
	return Action;
}

TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> FGitSourceControlRevision::GetBranchSource() const
{
	// if this revision was copied/moved from some other revision
	return BranchSource;
}

const FDateTime& FGitSourceControlRevision::GetDate() const
{
	return Date;
}

int32 FGitSourceControlRevision::GetCheckInIdentifier() const
{
	return CommitIdNumber;
}

int32 FGitSourceControlRevision::GetFileSize() const
{
	return FileSize;
}

#undef LOCTEXT_NAMESPACE
