// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISourceControlRevision.h"
#include "Misc/DateTime.h"

/** Revision of a file, linked to a specific commit */
class FGitSourceControlRevision : public ISourceControlRevision
{
public:
	FGitSourceControlRevision()
		: RevisionNumber(0)
	{
	}

	/** ISourceControlRevision interface */
	virtual bool Get(FString& InOutFilename, EConcurrency::Type InConcurrency = EConcurrency::Synchronous) const override;
	virtual bool GetAnnotated( TArray<FAnnotationLine>& OutLines ) const override;
	virtual bool GetAnnotated( FString& InOutFilename ) const override;
	virtual const FString& GetFilename() const override;
	virtual int32 GetRevisionNumber() const override;
	virtual const FString& GetRevision() const override;
	virtual const FString& GetDescription() const override;
	virtual const FString& GetUserName() const override;
	virtual const FString& GetClientSpec() const override;
	virtual const FString& GetAction() const override;
	virtual TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> GetBranchSource() const override;
	virtual const FDateTime& GetDate() const override;
	virtual int32 GetCheckInIdentifier() const override;
	virtual int32 GetFileSize() const override;

public:

	/** The filename this revision refers to */
	FString Filename;

	/** The full hexadecimal SHA1 id of the commit this revision refers to */
	FString CommitId;

	/** The short hexadecimal SHA1 id (8 first hex char out of 40) of the commit: the string to display */
	FString ShortCommitId;

	/** The numeric value of the short SHA1 (8 first hex char out of 40) */
	int32 CommitIdNumber;

	/** The index of the revision in the history (SBlueprintRevisionMenu assumes order for the "Depot" label) */
	int32 RevisionNumber;

	/** The SHA1 identifier of the file at this revision */
	FString FileHash;

	/** The description of this revision */
	FString Description;

	/** The user that made the change */
	FString UserName;

	/** The action (add, edit, branch etc.) performed at this revision */
	FString Action;

	/** Source of move ("branch" in Perforce term) if any */
	TSharedPtr<FGitSourceControlRevision, ESPMode::ThreadSafe> BranchSource;

	/** The date this revision was made */
	FDateTime Date;

	/** The size of the file at this revision */
	int32 FileSize;
};

/** History composed of the last 100 revisions of the file */
typedef TArray< TSharedRef<FGitSourceControlRevision, ESPMode::ThreadSafe> >	TGitSourceControlHistory;
