// Copyright Epic Games, Inc. All Rights Reserved.

#include "GitSourceControlSettings.h"
#include "Misc/ConfigCacheIni.h"
#include "SourceControlHelpers.h"

namespace GitSettingsConstants
{

/** The section of the ini file we load our settings from */
static const FString SettingsSection = TEXT("GitSourceControl.GitSourceControlSettings");

}

const FString FGitSourceControlSettings::GetBinaryPath() const
{
	FScopeLock ScopeLock(&CriticalSection);
	return BinaryPath;
}

bool FGitSourceControlSettings::SetBinaryPath(const FString& InString)
{
	FScopeLock ScopeLock(&CriticalSection);
	const bool bChanged = (BinaryPath != InString);
	if(bChanged)
	{
		BinaryPath = InString;
	}
	return bChanged;
}

// This is called at startup nearly before anything else in our module: BinaryPath will then be used by the provider
void FGitSourceControlSettings::LoadSettings()
{
	FScopeLock ScopeLock(&CriticalSection);
	const FString& IniFile = SourceControlHelpers::GetSettingsIni();
	GConfig->GetString(*GitSettingsConstants::SettingsSection, TEXT("BinaryPath"), BinaryPath, IniFile);
}

void FGitSourceControlSettings::SaveSettings() const
{
	FScopeLock ScopeLock(&CriticalSection);
	const FString& IniFile = SourceControlHelpers::GetSettingsIni();
	GConfig->SetString(*GitSettingsConstants::SettingsSection, TEXT("BinaryPath"), *BinaryPath, IniFile);
}
