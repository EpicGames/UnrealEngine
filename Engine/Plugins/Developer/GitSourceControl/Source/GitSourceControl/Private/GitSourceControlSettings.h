// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/CriticalSection.h"

struct FScriptContainerElement;

class FGitSourceControlSettings
{
public:
	/** Get the Git Binary Path */
	const FString GetBinaryPath() const;

	/** Set the Git Binary Path */
	bool SetBinaryPath(const FString& InString);

	/** Load settings from ini file */
	void LoadSettings();

	/** Save settings to ini file */
	void SaveSettings() const;

private:
	/** A critical section for settings access */
	mutable FCriticalSection CriticalSection;

	/** Git binary path */
	FString BinaryPath;
};
