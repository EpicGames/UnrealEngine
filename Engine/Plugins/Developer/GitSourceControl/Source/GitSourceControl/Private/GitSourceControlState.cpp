// Copyright Epic Games, Inc. All Rights Reserved.

#include "GitSourceControlState.h"
#include "RevisionControlStyle/RevisionControlStyle.h"
#include "Textures/SlateIcon.h"

#define LOCTEXT_NAMESPACE "GitSourceControl.State"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FGitSourceControlState::FGitSourceControlState(const FGitSourceControlState& Other) = default;
FGitSourceControlState::FGitSourceControlState(FGitSourceControlState&& Other) noexcept = default;
FGitSourceControlState& FGitSourceControlState::operator=(const FGitSourceControlState& Other) = default;
FGitSourceControlState& FGitSourceControlState::operator=(FGitSourceControlState&& Other) noexcept = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

int32 FGitSourceControlState::GetHistorySize() const
{
	return History.Num();
}

TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> FGitSourceControlState::GetHistoryItem( int32 HistoryIndex ) const
{
	check(History.IsValidIndex(HistoryIndex));
	return History[HistoryIndex];
}

TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> FGitSourceControlState::FindHistoryRevision( int32 RevisionNumber ) const
{
	for(const auto& Revision : History)
	{
		if(Revision->GetRevisionNumber() == RevisionNumber)
		{
			return Revision;
		}
	}

	return nullptr;
}

TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> FGitSourceControlState::FindHistoryRevision(const FString& InRevision) const
{
	// short hash must be >= 7 characters to have a reasonable probability of finding the correct revision
	if (!ensure(InRevision.Len() >= 7))
	{
		return nullptr;
	}
	
	for(const auto& Revision : History)
	{
		// support for short hashes
		const int32 Len = FMath::Min(Revision->CommitId.Len(), InRevision.Len());
		
		if(Revision->CommitId.Left(Len) == InRevision.Left(Len))
		{
			return Revision;
		}
	}

	return nullptr;
}

TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> FGitSourceControlState::GetCurrentRevision() const
{
	return nullptr;
}

ISourceControlState::FResolveInfo FGitSourceControlState::GetResolveInfo() const
{
	return PendingResolveInfo;
}

#if SOURCE_CONTROL_WITH_SLATE

FSlateIcon FGitSourceControlState::GetIcon() const
{
	switch (WorkingCopyState)
	{
	case EWorkingCopyState::Modified:
		return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.CheckedOut");
	case EWorkingCopyState::Added:
		return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.OpenForAdd");
	case EWorkingCopyState::Renamed:
	case EWorkingCopyState::Copied:
		return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Branched");
	case EWorkingCopyState::Deleted: // Deleted & Missing files does not show in Content Browser
	case EWorkingCopyState::Missing:
		return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.MarkedForDelete");
	case EWorkingCopyState::Conflicted:
		return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Conflicted");
	case EWorkingCopyState::NotControlled:
		return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.NotInDepot");
	case EWorkingCopyState::Unknown:
	case EWorkingCopyState::Unchanged: // Unchanged is the same as "Pristine" (not checked out) for Perforce, ie no icon
	case EWorkingCopyState::Ignored:
	default:
		return FSlateIcon();
	}
}

#endif //SOURCE_CONTROL_WITH_SLATE


FText FGitSourceControlState::GetDisplayName() const
{
	switch(WorkingCopyState)
	{
	case EWorkingCopyState::Unknown:
		return LOCTEXT("Unknown", "Unknown");
	case EWorkingCopyState::Unchanged:
		return LOCTEXT("Unchanged", "Unchanged");
	case EWorkingCopyState::Added:
		return LOCTEXT("Added", "Added");
	case EWorkingCopyState::Deleted:
		return LOCTEXT("Deleted", "Deleted");
	case EWorkingCopyState::Modified:
		return LOCTEXT("Modified", "Modified");
	case EWorkingCopyState::Renamed:
		return LOCTEXT("Renamed", "Renamed");
	case EWorkingCopyState::Copied:
		return LOCTEXT("Copied", "Copied");
	case EWorkingCopyState::Conflicted:
		return LOCTEXT("ContentsConflict", "Contents Conflict");
	case EWorkingCopyState::Ignored:
		return LOCTEXT("Ignored", "Ignored");
	case EWorkingCopyState::NotControlled:
		return LOCTEXT("NotControlled", "Not Under Revision Control");
	case EWorkingCopyState::Missing:
		return LOCTEXT("Missing", "Missing");
	}

	return FText();
}

FText FGitSourceControlState::GetDisplayTooltip() const
{
	switch(WorkingCopyState)
	{
	case EWorkingCopyState::Unknown:
		return LOCTEXT("Unknown_Tooltip", "Unknown revision control state");
	case EWorkingCopyState::Unchanged:
		return LOCTEXT("Pristine_Tooltip", "There are no modifications");
	case EWorkingCopyState::Added:
		return LOCTEXT("Added_Tooltip", "Item is scheduled for addition");
	case EWorkingCopyState::Deleted:
		return LOCTEXT("Deleted_Tooltip", "Item is scheduled for deletion");
	case EWorkingCopyState::Modified:
		return LOCTEXT("Modified_Tooltip", "Item has been modified");
	case EWorkingCopyState::Renamed:
		return LOCTEXT("Renamed_Tooltip", "Item has been renamed");
	case EWorkingCopyState::Copied:
		return LOCTEXT("Copied_Tooltip", "Item has been copied");
	case EWorkingCopyState::Conflicted:
		return LOCTEXT("ContentsConflict_Tooltip", "The contents of the item conflict with updates received from the repository.");
	case EWorkingCopyState::Ignored:
		return LOCTEXT("Ignored_Tooltip", "Item is being ignored.");
	case EWorkingCopyState::NotControlled:
		return LOCTEXT("NotControlled_Tooltip", "Item is not under version control.");
	case EWorkingCopyState::Missing:
		return LOCTEXT("Missing_Tooltip", "Item is missing (e.g., you moved or deleted it without using Git). This also indicates that a directory is incomplete (a checkout or update was interrupted).");
	}

	return FText();
}

const FString& FGitSourceControlState::GetFilename() const
{
	return LocalFilename;
}

const FDateTime& FGitSourceControlState::GetTimeStamp() const
{
	return TimeStamp;
}

// Deleted and Missing assets cannot appear in the Content Browser, but the do in the Submit files to Source Control window!
bool FGitSourceControlState::CanCheckIn() const
{
	return WorkingCopyState == EWorkingCopyState::Added
		|| WorkingCopyState == EWorkingCopyState::Deleted
		|| WorkingCopyState == EWorkingCopyState::Missing
		|| WorkingCopyState == EWorkingCopyState::Modified
		|| WorkingCopyState == EWorkingCopyState::Renamed;
}

bool FGitSourceControlState::CanCheckout() const
{
	return false; // With Git all tracked files in the working copy are always already checked-out (as opposed to Perforce)
}

bool FGitSourceControlState::IsCheckedOut() const
{
	return IsSourceControlled(); // With Git all tracked files in the working copy are always checked-out (as opposed to Perforce)
}

bool FGitSourceControlState::IsCheckedOutOther(FString* Who) const
{
	return false; // Git does not lock checked-out files as Perforce does
}

bool FGitSourceControlState::IsCurrent() const
{
	return true; // @todo check the state of the HEAD versus the state of tracked branch on remote
}

bool FGitSourceControlState::IsSourceControlled() const
{
	return WorkingCopyState != EWorkingCopyState::NotControlled && WorkingCopyState != EWorkingCopyState::Ignored && WorkingCopyState != EWorkingCopyState::Unknown;
}

bool FGitSourceControlState::IsAdded() const
{
	return WorkingCopyState == EWorkingCopyState::Added;
}

bool FGitSourceControlState::IsDeleted() const
{
	return WorkingCopyState == EWorkingCopyState::Deleted || WorkingCopyState == EWorkingCopyState::Missing;
}

bool FGitSourceControlState::IsIgnored() const
{
	return WorkingCopyState == EWorkingCopyState::Ignored;
}

bool FGitSourceControlState::CanEdit() const
{
	return true; // With Git all files in the working copy are always editable (as opposed to Perforce)
}

bool FGitSourceControlState::CanDelete() const
{
	return IsSourceControlled() && IsCurrent();
}

bool FGitSourceControlState::IsUnknown() const
{
	return WorkingCopyState == EWorkingCopyState::Unknown;
}

bool FGitSourceControlState::IsModified() const
{
	// Warning: for Perforce, a checked-out file is locked for modification (whereas with Git all tracked files are checked-out),
	// so for a clean "check-in" (commit) checked-out files unmodified should be removed from the changeset (the index)
	// http://stackoverflow.com/questions/12357971/what-does-revert-unchanged-files-mean-in-perforce
	//
	// Thus, before check-in UE Editor call RevertUnchangedFiles() in PromptForCheckin() and CheckinFiles().
	//
	// So here we must take care to enumerate all states that need to be commited,
	// all other will be discarded :
	//  - Unknown
	//  - Unchanged
	//  - NotControlled
	//  - Ignored
	return WorkingCopyState == EWorkingCopyState::Added
		|| WorkingCopyState == EWorkingCopyState::Deleted
		|| WorkingCopyState == EWorkingCopyState::Modified
		|| WorkingCopyState == EWorkingCopyState::Renamed
		|| WorkingCopyState == EWorkingCopyState::Copied
		|| WorkingCopyState == EWorkingCopyState::Conflicted
		|| WorkingCopyState == EWorkingCopyState::Missing;
}


bool FGitSourceControlState::CanAdd() const
{
	return WorkingCopyState == EWorkingCopyState::NotControlled;
}

bool FGitSourceControlState::IsConflicted() const
{
	return WorkingCopyState == EWorkingCopyState::Conflicted;
}

bool FGitSourceControlState::CanRevert() const
{
	return CanCheckIn();
}

#undef LOCTEXT_NAMESPACE
