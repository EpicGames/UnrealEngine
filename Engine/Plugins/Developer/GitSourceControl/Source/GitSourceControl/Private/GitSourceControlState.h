// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GitSourceControlRevision.h"

namespace EWorkingCopyState
{
	enum Type
	{
		Unknown,
		Unchanged, // called "clean" in SVN, "Pristine" in Perforce
		Added,
		Deleted,
		Modified,
		Renamed,
		Copied,
		Missing,
		Conflicted,
		NotControlled,
		Ignored,
	};
}

class FGitSourceControlState : public ISourceControlState
{
public:
	FGitSourceControlState( const FString& InLocalFilename )
		: LocalFilename(InLocalFilename)
		, WorkingCopyState(EWorkingCopyState::Unknown)
		, TimeStamp(0)
	{
	}

	FGitSourceControlState(const FGitSourceControlState& Other);
	FGitSourceControlState(FGitSourceControlState&& Other) noexcept;
	FGitSourceControlState& operator=(const FGitSourceControlState& Other);
	FGitSourceControlState& operator=(FGitSourceControlState&& Other) noexcept;

	/** ISourceControlState interface */
	virtual int32 GetHistorySize() const override;
	virtual TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> GetHistoryItem(int32 HistoryIndex) const override;
	virtual TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> FindHistoryRevision(int32 RevisionNumber) const override;
	virtual TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> FindHistoryRevision(const FString& InRevision) const override;
	virtual TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> GetCurrentRevision() const override;
	virtual FResolveInfo GetResolveInfo() const override;
#if SOURCE_CONTROL_WITH_SLATE
	virtual FSlateIcon GetIcon() const override;
#endif //SOURCE_CONTROL_WITH_SLATE
	virtual FText GetDisplayName() const override;
	virtual FText GetDisplayTooltip() const override;
	virtual const FString& GetFilename() const override;
	virtual const FDateTime& GetTimeStamp() const override;
	virtual bool CanCheckIn() const override;
	virtual bool CanCheckout() const override;
	virtual bool IsCheckedOut() const override;
	virtual bool IsCheckedOutOther(FString* Who = nullptr) const override;
	virtual bool IsCheckedOutInOtherBranch(const FString& CurrentBranch = FString()) const override { return false;  }
	virtual bool IsModifiedInOtherBranch(const FString& CurrentBranch = FString()) const override { return false; }
	virtual bool IsCheckedOutOrModifiedInOtherBranch(const FString& CurrentBranch = FString()) const override { return IsCheckedOutInOtherBranch(CurrentBranch) || IsModifiedInOtherBranch(CurrentBranch); }
	virtual TArray<FString> GetCheckedOutBranches() const override { return TArray<FString>(); }
	virtual FString GetOtherUserBranchCheckedOuts() const override { return FString(); }
	virtual bool GetOtherBranchHeadModification(FString& HeadBranchOut, FString& ActionOut, int32& HeadChangeListOut) const override { return false; }
	virtual bool IsCurrent() const override;
	virtual bool IsSourceControlled() const override;
	virtual bool IsAdded() const override;
	virtual bool IsDeleted() const override;
	virtual bool IsIgnored() const override;
	virtual bool CanEdit() const override;
	virtual bool IsUnknown() const override;
	virtual bool IsModified() const override;
	virtual bool CanAdd() const override;
	virtual bool CanDelete() const override;
	virtual bool IsConflicted() const override;
	virtual bool CanRevert() const override;

public:
	/** History of the item, if any */
	TGitSourceControlHistory History;

	/** Filename on disk */
	FString LocalFilename;

	/** Pending rev info with which a file must be resolved, invalid if no resolve pending */
	FResolveInfo PendingResolveInfo;

	UE_DEPRECATED(5.3, "Use PendingResolveInfo.BaseRevision instead")
	FString PendingMergeBaseFileHash;

	/** State of the working copy */
	EWorkingCopyState::Type WorkingCopyState;

	/** The timestamp of the last update */
	FDateTime TimeStamp;
};
