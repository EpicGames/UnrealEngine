// Copyright Epic Games, Inc. All Rights Reserved.

#include "GitSourceControlUtils.h"
#include "GitSourceControlCommand.h"
#include "GitSourceControlState.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "ISourceControlModule.h"
#include "GitSourceControlModule.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#include "Apple/ScopeAutoreleasePool.h"
#endif

#if PLATFORM_LINUX
#endif


namespace GitSourceControlConstants
{
	/** The maximum number of files we submit in a single Git command */
	const int32 MaxFilesPerBatch = 50;
}

FGitScopedTempFile::FGitScopedTempFile(const FText& InText)
{
	Filename = FPaths::CreateTempFilename(*FPaths::ProjectLogDir(), TEXT("Git-Temp"), TEXT(".txt"));
	if(!FFileHelper::SaveStringToFile(InText.ToString(), *Filename, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM))
	{
		UE_LOGF(LogSourceControl, Error, "Failed to write to temp file: %ls", *Filename);
	}
}

FGitScopedTempFile::~FGitScopedTempFile()
{
	if(FPaths::FileExists(Filename))
	{
		if(!FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*Filename))
		{
			UE_LOGF(LogSourceControl, Error, "Failed to delete temp file: %ls", *Filename);
		}
	}
}

const FString& FGitScopedTempFile::GetFilename() const
{
	return Filename;
}


namespace GitSourceControlUtils
{

// Launch the Git command line process and extract its results & errors
static bool RunCommandInternalRaw(const FString& InCommand, const FString& InPathToGitBinary, const FString& InRepositoryRoot, const TArray<FString>& InParameters, const TArray<FString>& InFiles, FString& OutResults, FString& OutErrors)
{
	int32 ReturnCode = 0;
	FString FullCommand;
	FString LogableCommand; // short version of the command for logging purpose

	if(!InRepositoryRoot.IsEmpty())
	{
		FString RepositoryRoot = InRepositoryRoot;

		// Detect a "migrate asset" scenario (a "git add" command is applied to files outside the current project)
		if ( (InFiles.Num() > 0) && !FPaths::IsRelative(InFiles[0]) && !InFiles[0].StartsWith(InRepositoryRoot) )
		{
			// in this case, find the git repository (if any) of the destination Project
			FString DestinationRepositoryRoot;
			if(FindRootDirectory(FPaths::GetPath(InFiles[0]), DestinationRepositoryRoot))
			{
				RepositoryRoot = DestinationRepositoryRoot; // if found use it for the "add" command (else not, to avoid producing one more error in logs)
			}
		}

		// Specify the working copy (the root) of the git repository (before the command itself)
		FullCommand  = TEXT("-C \"");
		FullCommand += RepositoryRoot;
		FullCommand += TEXT("\" ");
	}
	// then the git command itself ("status", "log", "commit"...)
	LogableCommand += InCommand;

	// Append to the command all parameters, and then finally the files
	for(const auto& Parameter : InParameters)
	{
		LogableCommand += TEXT(" ");
		LogableCommand += Parameter;
	}
	for(const auto& File : InFiles)
	{
		LogableCommand += TEXT(" \"");
		LogableCommand += File;
		LogableCommand += TEXT("\"");
	}
	// Also, Git does not have a "--non-interactive" option, as it auto-detects when there are no connected standard input/output streams

	FullCommand += LogableCommand;

#if UE_BUILD_DEBUG
	UE_LOGF(LogSourceControl, Log, "RunCommandInternalRaw: 'git %ls'", *LogableCommand);
#endif

	FString PathToGitOrEnvBinary = InPathToGitBinary;
#if PLATFORM_MAC
	// The Cocoa application does not inherit shell environment variables, so add the path expected to have git-lfs to PATH
	FString PathEnv = FPlatformMisc::GetEnvironmentVariable(TEXT("PATH"));
	FString GitInstallPath = FPaths::GetPath(InPathToGitBinary);

	TArray<FString> PathArray;
	PathEnv.ParseIntoArray(PathArray, FPlatformMisc::GetPathVarDelimiter());
	bool bHasGitInstallPath = false;
	for (auto Path : PathArray)
	{
		if (GitInstallPath.Equals(Path, ESearchCase::CaseSensitive))
		{
			bHasGitInstallPath = true;
			break;
		}
	}

	if (!bHasGitInstallPath)
	{
		PathToGitOrEnvBinary = FString("/usr/bin/env");
		FullCommand = FString::Printf(TEXT("PATH=\"%s%s%s\" \"%s\" %s"), *GitInstallPath, FPlatformMisc::GetPathVarDelimiter(), *PathEnv, *InPathToGitBinary, *FullCommand);
	}
#endif
	FPlatformProcess::ExecProcess(*PathToGitOrEnvBinary, *FullCommand, &ReturnCode, &OutResults, &OutErrors);

#if UE_BUILD_DEBUG

	if (OutResults.IsEmpty())
	{
		UE_LOGF(LogSourceControl, Log, "RunCommandInternalRaw: 'OutResults=n/a'");
	}
	else
	{
		UE_LOGF(LogSourceControl, Log, "RunCommandInternalRaw(%ls): OutResults=\n%ls", *InCommand, *OutResults);
	}

	if(ReturnCode != 0)
	{
		if (OutErrors.IsEmpty())
		{
			UE_LOGF(LogSourceControl, Warning, "RunCommandInternalRaw: 'OutErrors=n/a'");
		}
		else
		{
			UE_LOGF(LogSourceControl, Warning, "RunCommandInternalRaw(%ls): OutErrors=\n%ls", *InCommand, *OutErrors);
		}
	}
#endif

	return ReturnCode == 0;
}

// Basic parsing or results & errors from the Git command line process
static bool RunCommandInternal(const FString& InCommand, const FString& InPathToGitBinary, const FString& InRepositoryRoot, const TArray<FString>& InParameters, const TArray<FString>& InFiles, TArray<FString>& OutResults, TArray<FString>& OutErrorMessages)
{
	bool bResult;
	FString Results;
	FString Errors;

	bResult = RunCommandInternalRaw(InCommand, InPathToGitBinary, InRepositoryRoot, InParameters, InFiles, Results, Errors);
	Results.ParseIntoArray(OutResults, TEXT("\n"), true);
	Errors.ParseIntoArray(OutErrorMessages, TEXT("\n"), true);

	return bResult;
}

FString FindGitBinaryPath()
{
#if PLATFORM_WINDOWS
	// 1) First of all, look into standard install directories
	// NOTE using only "git" (or "git.exe") relying on the "PATH" envvar does not always work as expected, depending on the installation:
	// If the PATH is set with "git/cmd" instead of "git/bin",
	// "git.exe" launch "git/cmd/git.exe" that redirect to "git/bin/git.exe" and ExecProcess() is unable to catch its outputs streams.
	// First check the 64-bit program files directory:
	FString GitBinaryPath(TEXT("C:/Program Files/Git/bin/git.exe"));
	bool bFound = CheckGitAvailability(GitBinaryPath);
	if(!bFound)
	{
		// otherwise check the 32-bit program files directory.
		GitBinaryPath = TEXT("C:/Program Files (x86)/Git/bin/git.exe");
		bFound = CheckGitAvailability(GitBinaryPath);
	}
	if(!bFound)
	{
		// else the install dir for the current user: C:\Users\UserName\AppData\Local\Programs\Git\cmd
		FString AppDataLocalPath = FPlatformMisc::GetEnvironmentVariable(TEXT("LOCALAPPDATA"));
		GitBinaryPath = FString::Printf(TEXT("%s/Programs/Git/cmd/git.exe"), *AppDataLocalPath);
		bFound = CheckGitAvailability(GitBinaryPath);
	}

	// 2) Else, look for the version of Git bundled with SmartGit "Installer with JRE"
	if(!bFound)
	{
		GitBinaryPath = TEXT("C:/Program Files (x86)/SmartGit/git/bin/git.exe");
		bFound = CheckGitAvailability(GitBinaryPath);
		if (!bFound)
		{
			// If git is not found in "git/bin/" subdirectory, try the "bin/" path that was in use before
			GitBinaryPath = TEXT("C:/Program Files (x86)/SmartGit/bin/git.exe");
			bFound = CheckGitAvailability(GitBinaryPath);
		}
	}

	// 3) Else, look for the local_git provided by SourceTree
	if(!bFound)
	{
		// C:\Users\UserName\AppData\Local\Atlassian\SourceTree\git_local\bin
		FString AppDataLocalPath = FPlatformMisc::GetEnvironmentVariable(TEXT("LOCALAPPDATA"));
		GitBinaryPath = FString::Printf(TEXT("%s/Atlassian/SourceTree/git_local/bin/git.exe"), *AppDataLocalPath);
		bFound = CheckGitAvailability(GitBinaryPath);
	}

	// 4) Else, look for the PortableGit provided by GitHub Desktop
	if(!bFound)
	{
		// The latest GitHub Desktop adds its binaries into the local appdata directory:
		// C:\Users\UserName\AppData\Local\GitHub\PortableGit_c2ba306e536fdf878271f7fe636a147ff37326ad\cmd
		FString AppDataLocalPath = FPlatformMisc::GetEnvironmentVariable(TEXT("LOCALAPPDATA"));
		FString SearchPath = FString::Printf(TEXT("%s/GitHub/PortableGit_*"), *AppDataLocalPath);
		TArray<FString> PortableGitFolders;
		IFileManager::Get().FindFiles(PortableGitFolders, *SearchPath, false, true);
		if(PortableGitFolders.Num() > 0)
		{
			// FindFiles just returns directory names, so we need to prepend the root path to get the full path.
			GitBinaryPath = FString::Printf(TEXT("%s/GitHub/%s/cmd/git.exe"), *AppDataLocalPath, *(PortableGitFolders.Last())); // keep only the last PortableGit found
			bFound = CheckGitAvailability(GitBinaryPath);
			if (!bFound)
			{
				// If Portable git is not found in "cmd/" subdirectory, try the "bin/" path that was in use before
				GitBinaryPath = FString::Printf(TEXT("%s/GitHub/%s/bin/git.exe"), *AppDataLocalPath, *(PortableGitFolders.Last())); // keep only the last PortableGit found
				bFound = CheckGitAvailability(GitBinaryPath);
			}
		}
	}

	// 5) Else, look for the version of Git bundled with Tower
	if (!bFound)
	{
		GitBinaryPath = TEXT("C:/Program Files (x86)/fournova/Tower/vendor/Git/bin/git.exe");
		bFound = CheckGitAvailability(GitBinaryPath);
	}

#elif PLATFORM_MAC
	// 1) First of all, look for the version of git provided by official git
	FString GitBinaryPath = TEXT("/usr/local/git/bin/git");
	bool bFound = CheckGitAvailability(GitBinaryPath);

	// 2) Else, look for the version of git provided by Homebrew
	if (!bFound)
	{
		GitBinaryPath = TEXT("/usr/local/bin/git");
		bFound = CheckGitAvailability(GitBinaryPath);
	}

	// 3) Else, look for the version of git provided by MacPorts
	if (!bFound)
	{
		GitBinaryPath = TEXT("/opt/local/bin/git");
		bFound = CheckGitAvailability(GitBinaryPath);
	}

	// 4) Else, look for the version of git provided by Command Line Tools
	if (!bFound)
	{
		GitBinaryPath = TEXT("/usr/bin/git");
		bFound = CheckGitAvailability(GitBinaryPath);
	}

	{
		SCOPED_AUTORELEASE_POOL;
		NSWorkspace* SharedWorkspace = [NSWorkspace sharedWorkspace];

		// 5) Else, look for the version of local_git provided by SmartGit
		if (!bFound)
		{
			NSURL* AppURL = [SharedWorkspace URLForApplicationWithBundleIdentifier:@"com.syntevo.smartgit"];
			if (AppURL != nullptr)
			{
				NSBundle* Bundle = [NSBundle bundleWithURL:AppURL];
				GitBinaryPath = FString::Printf(TEXT("%s/git/bin/git"), *FString([Bundle resourcePath]));
				bFound = CheckGitAvailability(GitBinaryPath);
			}
		}

		// 6) Else, look for the version of local_git provided by SourceTree
		if (!bFound)
		{
			NSURL* AppURL = [SharedWorkspace URLForApplicationWithBundleIdentifier:@"com.torusknot.SourceTreeNotMAS"];
			if (AppURL != nullptr)
			{
				NSBundle* Bundle = [NSBundle bundleWithURL:AppURL];
				GitBinaryPath = FString::Printf(TEXT("%s/git_local/bin/git"), *FString([Bundle resourcePath]));
				bFound = CheckGitAvailability(GitBinaryPath);
			}
		}

		// 7) Else, look for the version of local_git provided by GitHub Desktop
		if (!bFound)
		{
			NSURL* AppURL = [SharedWorkspace URLForApplicationWithBundleIdentifier:@"com.github.GitHubClient"];
			if (AppURL != nullptr)
			{
				NSBundle* Bundle = [NSBundle bundleWithURL:AppURL];
				GitBinaryPath = FString::Printf(TEXT("%s/app/git/bin/git"), *FString([Bundle resourcePath]));
				bFound = CheckGitAvailability(GitBinaryPath);
			}
		}

		// 8) Else, look for the version of local_git provided by Tower2
		if (!bFound)
		{
			NSURL* AppURL = [SharedWorkspace URLForApplicationWithBundleIdentifier:@"com.fournova.Tower2"];
			if (AppURL != nullptr)
			{
				NSBundle* Bundle = [NSBundle bundleWithURL:AppURL];
				GitBinaryPath = FString::Printf(TEXT("%s/git/bin/git"), *FString([Bundle resourcePath]));
				bFound = CheckGitAvailability(GitBinaryPath);
			}
		}
	}

#else
	FString GitBinaryPath = TEXT("/usr/bin/git");
	bool bFound = CheckGitAvailability(GitBinaryPath);
#endif

	if(bFound)
	{
		FPaths::MakePlatformFilename(GitBinaryPath);
	}
	else
	{
		// If we did not find a path to Git, set it empty
		GitBinaryPath.Empty();
	}

	return GitBinaryPath;
}

bool CheckGitAvailability(const FString& InPathToGitBinary, FGitVersion *OutVersion)
{
#if PLATFORM_WINDOWS
	const TCHAR* GitExecutableSuffix = TEXT("git.exe");
#else
	const TCHAR* GitExecutableSuffix = TEXT("git");
#endif

	// Make sure we don't run arbitrary executables
	if ( !InPathToGitBinary.EndsWith(GitExecutableSuffix) )
	{
		return false;
	}

	FString InfoMessages;
	FString ErrorMessages;
	bool bGitAvailable = RunCommandInternalRaw(TEXT("version"), InPathToGitBinary, FString(), TArray<FString>(), TArray<FString>(), InfoMessages, ErrorMessages);
	if(bGitAvailable)
	{
		if(!InfoMessages.Contains("git"))
		{
			bGitAvailable = false;
		}
		else if(OutVersion)
		{
			ParseGitVersion(InfoMessages, OutVersion);
			FindGitCapabilities(InPathToGitBinary, OutVersion);
			FindGitLfsCapabilities(InPathToGitBinary, OutVersion);
		}
	}

	return bGitAvailable;
}

void ParseGitVersion(const FString& InVersionString, FGitVersion *OutVersion) 
{
	// Parse "git version 2.11.0" into the string tokens "git", "version", "2.11.0"
	TArray<FString> TokenizedString;
	InVersionString.ParseIntoArrayWS(TokenizedString);

	// Select the string token containing the version "2.11.0"
	const FString* TokenVersionStringPtr = TokenizedString.FindByPredicate([](FString& s) { return TChar<TCHAR>::IsDigit(s[0]); });
	if(TokenVersionStringPtr)
	{
		// Parse the version into its two Major.Minor numerical components
		TArray<FString> ParsedVersionString;
		TokenVersionStringPtr->ParseIntoArray(ParsedVersionString, TEXT("."));
		if(ParsedVersionString.Num() >= 2)
		{
			if(ParsedVersionString[0].IsNumeric() && ParsedVersionString[1].IsNumeric())
			{
				OutVersion->Major = FCString::Atoi(*ParsedVersionString[0]);
				OutVersion->Minor = FCString::Atoi(*ParsedVersionString[1]);
			}
		}
	}
}

void FindGitCapabilities(const FString& InPathToGitBinary, FGitVersion *OutVersion)
{
	FString InfoMessages;
	FString ErrorMessages;
	RunCommandInternalRaw(TEXT("cat-file -h"), InPathToGitBinary, FString(), TArray<FString>(), TArray<FString>(), InfoMessages, ErrorMessages);
	if (InfoMessages.Contains("--filters"))
	{
		OutVersion->bHasCatFileWithFilters = true;
	}
}

void FindGitLfsCapabilities(const FString& InPathToGitBinary, FGitVersion *OutVersion)
{
	FString InfoMessages;
	FString ErrorMessages;
	bool bGitLfsAvailable = RunCommandInternalRaw(TEXT("lfs version"), InPathToGitBinary, FString(), TArray<FString>(), TArray<FString>(), InfoMessages, ErrorMessages);
	if(bGitLfsAvailable)
	{
		OutVersion->bHasGitLfs = true;

		if(0 <= InfoMessages.Compare(TEXT("git-lfs/2.0.0")))
		{
			OutVersion->bHasGitLfsLocking = true; // Git LFS File Locking workflow introduced in "git-lfs/2.0.0"
		}
	}
}

// Find the root of the Git repository, looking from the provided path and upward in its parent directories.
bool FindRootDirectory(const FString& InPath, FString& OutRepositoryRoot)
{
	bool bFound = false;
	FString PathToGitSubdirectory;
	OutRepositoryRoot = InPath;

	auto TrimTrailing = [](FString& Str, const TCHAR Char)
	{
		int32 Len = Str.Len();
		while(Len && Str[Len - 1] == Char)
		{
			Str.LeftChopInline(1);
			Len = Str.Len();
		}
	};

	TrimTrailing(OutRepositoryRoot, '\\');
	TrimTrailing(OutRepositoryRoot, '/');

	while(!bFound && !OutRepositoryRoot.IsEmpty())
	{
		// Look for the ".git" subdirectory (or file) present at the root of every Git repository
		PathToGitSubdirectory = OutRepositoryRoot / TEXT(".git");
		bFound = IFileManager::Get().DirectoryExists(*PathToGitSubdirectory) || IFileManager::Get().FileExists(*PathToGitSubdirectory);
		if(!bFound)
		{
			int32 LastSlashIndex;
			if(OutRepositoryRoot.FindLastChar('/', LastSlashIndex))
			{
				OutRepositoryRoot.LeftInline(LastSlashIndex);
			}
			else
			{
				OutRepositoryRoot.Empty();
			}
		}
	}
	if(!bFound)
	{
		OutRepositoryRoot = InPath; // If not found, return the provided dir as best possible root.
	}
	return bFound;
}

void GetUserConfig(const FString& InPathToGitBinary, const FString& InRepositoryRoot, FString& OutUserName, FString& OutUserEmail)
{
	bool bResults;
	TArray<FString> InfoMessages;
	TArray<FString> ErrorMessages;
	TArray<FString> Parameters;
	Parameters.Add(TEXT("user.name"));
	bResults = RunCommandInternal(TEXT("config"), InPathToGitBinary, InRepositoryRoot, Parameters, TArray<FString>(), InfoMessages, ErrorMessages);
	if(bResults && InfoMessages.Num() > 0)
	{
		OutUserName = InfoMessages[0];
	}

	Parameters.Reset();
	Parameters.Add(TEXT("user.email"));
	InfoMessages.Reset();
	bResults &= RunCommandInternal(TEXT("config"), InPathToGitBinary, InRepositoryRoot, Parameters, TArray<FString>(), InfoMessages, ErrorMessages);
	if(bResults && InfoMessages.Num() > 0)
	{
		OutUserEmail = InfoMessages[0];
	}
}

bool GetBranchName(const FString& InPathToGitBinary, const FString& InRepositoryRoot, FString& OutBranchName)
{
	bool bResults;
	TArray<FString> InfoMessages;
	TArray<FString> ErrorMessages;
	TArray<FString> Parameters;
	Parameters.Add(TEXT("--short"));
	Parameters.Add(TEXT("--quiet"));		// no error message while in detached HEAD
	Parameters.Add(TEXT("HEAD"));
	bResults = RunCommandInternal(TEXT("symbolic-ref"), InPathToGitBinary, InRepositoryRoot, Parameters, TArray<FString>(), InfoMessages, ErrorMessages);
	if(bResults && InfoMessages.Num() > 0)
	{
		OutBranchName = InfoMessages[0];
	}
	else
	{
		Parameters.Reset();
		Parameters.Add(TEXT("-1"));
		Parameters.Add(TEXT("--format=\"%h\""));		// no error message while in detached HEAD
		bResults = RunCommandInternal(TEXT("log"), InPathToGitBinary, InRepositoryRoot, Parameters, TArray<FString>(), InfoMessages, ErrorMessages);
		if(bResults && InfoMessages.Num() > 0)
		{
			OutBranchName = "HEAD detached at ";
			OutBranchName += InfoMessages[0];
		}
		else
		{
			bResults = false;
		}
	}

	return bResults;
}

bool GetRemoteUrl(const FString& InPathToGitBinary, const FString& InRepositoryRoot, FString& OutRemoteUrl)
{
	TArray<FString> InfoMessages;
	TArray<FString> ErrorMessages;
	TArray<FString> Parameters;
	Parameters.Add(TEXT("get-url"));
	Parameters.Add(TEXT("origin"));
	const bool bResults = RunCommandInternal(TEXT("remote"), InPathToGitBinary, InRepositoryRoot, Parameters, TArray<FString>(), InfoMessages, ErrorMessages);
	if (bResults && InfoMessages.Num() > 0)
	{
		OutRemoteUrl = InfoMessages[0];
	}

	return bResults;
}

bool RunCommand(const FString& InCommand, const FString& InPathToGitBinary, const FString& InRepositoryRoot, const TArray<FString>& InParameters, const TArray<FString>& InFiles, TArray<FString>& OutResults, TArray<FString>& OutErrorMessages)
{
	bool bResult = true;

	if(InFiles.Num() > GitSourceControlConstants::MaxFilesPerBatch)
	{
		// Batch files up so we dont exceed command-line limits
		int32 FileCount = 0;
		while(FileCount < InFiles.Num())
		{
			TArray<FString> FilesInBatch;
			for(int32 FileIndex = 0; FileCount < InFiles.Num() && FileIndex < GitSourceControlConstants::MaxFilesPerBatch; FileIndex++, FileCount++)
			{
				FilesInBatch.Add(InFiles[FileCount]);
			}

			TArray<FString> BatchResults;
			TArray<FString> BatchErrors;
			bResult &= RunCommandInternal(InCommand, InPathToGitBinary, InRepositoryRoot, InParameters, FilesInBatch, BatchResults, BatchErrors);
			OutResults += BatchResults;
			OutErrorMessages += BatchErrors;
		}
	}
	else
	{
		bResult &= RunCommandInternal(InCommand, InPathToGitBinary, InRepositoryRoot, InParameters, InFiles, OutResults, OutErrorMessages);
	}

	return bResult;
}

// Run a Git "commit" command by batches
bool RunCommit(const FString& InPathToGitBinary, const FString& InRepositoryRoot, const TArray<FString>& InParameters, const TArray<FString>& InFiles, TArray<FString>& OutResults, TArray<FString>& OutErrorMessages)
{
	bool bResult = true;

	if(InFiles.Num() > GitSourceControlConstants::MaxFilesPerBatch)
	{
		// Batch files up so we dont exceed command-line limits
		int32 FileCount = 0;
		{
			TArray<FString> FilesInBatch;
			for(int32 FileIndex = 0; FileIndex < GitSourceControlConstants::MaxFilesPerBatch; FileIndex++, FileCount++)
			{
				FilesInBatch.Add(InFiles[FileCount]);
			}
			// First batch is a simple "git commit" command with only the first files
			bResult &= RunCommandInternal(TEXT("commit"), InPathToGitBinary, InRepositoryRoot, InParameters, FilesInBatch, OutResults, OutErrorMessages);
		}

		TArray<FString> Parameters;
		for(const auto& Parameter : InParameters)
		{
			Parameters.Add(Parameter);
		}
		Parameters.Add(TEXT("--amend"));

		while(FileCount < InFiles.Num())
		{
			TArray<FString> FilesInBatch;
			for(int32 FileIndex = 0; FileCount < InFiles.Num() && FileIndex < GitSourceControlConstants::MaxFilesPerBatch; FileIndex++, FileCount++)
			{
				FilesInBatch.Add(InFiles[FileCount]);
			}
			// Next batches "amend" the commit with some more files
			TArray<FString> BatchResults;
			TArray<FString> BatchErrors;
			bResult &= RunCommandInternal(TEXT("commit"), InPathToGitBinary, InRepositoryRoot, Parameters, FilesInBatch, BatchResults, BatchErrors);
			OutResults += BatchResults;
			OutErrorMessages += BatchErrors;
		}
	}
	else
	{
		bResult = RunCommandInternal(TEXT("commit"), InPathToGitBinary, InRepositoryRoot, InParameters, InFiles, OutResults, OutErrorMessages);
	}

	return bResult;
}

/**
 * @brief Extract the relative filename from a Git status result.
 *
 * Examples of status results:
M  Content/Textures/T_Perlin_Noise_M.uasset
R  Content/Textures/T_Perlin_Noise_M.uasset -> Content/Textures/T_Perlin_Noise_M2.uasset
?? Content/Materials/M_Basic_Wall.uasset
!! BasicCode.sln
 *
 * @param[in] InResult One line of status
 * @return Relative filename extracted from the line of status
 *
 * @see FGitStatusFileMatcher and StateFromGitStatus()
 */
static FString FilenameFromGitStatus(const FString& InResult)
{
	int32 RenameIndex;
	if(InResult.FindLastChar('>', RenameIndex))
	{
		// Extract only the second part of a rename "from -> to"
		return InResult.RightChop(RenameIndex + 2);
	}
	else
	{
		// Extract the relative filename from the Git status result (after the 2 letters status and 1 space)
		return InResult.RightChop(3);
	}
}

/** Match the relative filename of a Git status result with a provided absolute filename */
class FGitStatusFileMatcher
{
public:
	FGitStatusFileMatcher(const FString& InAbsoluteFilename)
		: AbsoluteFilename(InAbsoluteFilename)
	{
	}

	bool operator()(const FString& InResult) const
	{
		return AbsoluteFilename.Contains(FilenameFromGitStatus(InResult));
	}

private:
	const FString& AbsoluteFilename;
};

/**
 * Extract and interpret the file state from the given Git status result.
 * @see http://git-scm.com/docs/git-status
 * ' ' = unmodified
 * 'M' = modified
 * 'A' = added
 * 'D' = deleted
 * 'R' = renamed
 * 'C' = copied
 * 'U' = updated but unmerged
 * '?' = unknown/untracked
 * '!' = ignored
*/
class FGitStatusParser
{
public:
	FGitStatusParser(const FString& InResult)
	{
		TCHAR IndexState = InResult[0];
		TCHAR WCopyState = InResult[1];
		if(   (IndexState == 'U' || WCopyState == 'U')
		   || (IndexState == 'A' && WCopyState == 'A')
		   || (IndexState == 'D' && WCopyState == 'D'))
		{
			// "Unmerged" conflict cases are generally marked with a "U",
			// but there are also the special cases of both "A"dded, or both "D"eleted
			State = EWorkingCopyState::Conflicted;
		}
		else if(IndexState == 'A')
		{
			State = EWorkingCopyState::Added;
		}
		else if(IndexState == 'D')
		{
			State = EWorkingCopyState::Deleted;
		}
		else if(WCopyState == 'D')
		{
			State = EWorkingCopyState::Missing;
		}
		else if(IndexState == 'M' || WCopyState == 'M')
		{
			State = EWorkingCopyState::Modified;
		}
		else if(IndexState == 'R')
		{
			State = EWorkingCopyState::Renamed;
		}
		else if(IndexState == 'C')
		{
			State = EWorkingCopyState::Copied;
		}
		else if(IndexState == '?' || WCopyState == '?')
		{
			State = EWorkingCopyState::NotControlled;
		}
		else if(IndexState == '!' || WCopyState == '!')
		{
			State = EWorkingCopyState::Ignored;
		}
		else
		{
			// Unmodified never yield a status
			State = EWorkingCopyState::Unknown;
		}
	}

	EWorkingCopyState::Type State;
};

/**
 * Extract the status of a unmerged (conflict) file
 *
 * Example output of git ls-files --unmerged Content/Blueprints/BP_Test.uasset
100644 d9b33098273547b57c0af314136f35b494e16dcb 1	Content/Blueprints/BP_Test.uasset
100644 a14347dc3b589b78fb19ba62a7e3982f343718bc 2	Content/Blueprints/BP_Test.uasset
100644 f3137a7167c840847cd7bd2bf07eefbfb2d9bcd2 3	Content/Blueprints/BP_Test.uasset
 *
 * 1: The "common ancestor" of the file (the version of the file that both the current and other branch originated from).
 * 2: The version from the current branch (the master branch in this case).
 * 3: The version from the other branch (the test branch)
*/
class FGitConflictStatusParser
{
public:
	/** Parse the unmerge status: extract the base SHA1 identifier of the file */
	FGitConflictStatusParser(const TArray<FString>& InResults)
	{
		const FString& CommonAncestor = InResults[0]; // 1: The common ancestor of merged branches
		CommonAncestorFileId = CommonAncestor.Mid(7, 40);
		CommonAncestorFilename = CommonAncestor.Right(50);

		if (ensure(InResults.IsValidIndex(2)))
		{
			const FString& RemoteBranch = InResults[2]; // 1: The common ancestor of merged branches
			RemoteFileId = RemoteBranch.Mid(7, 40);
			RemoteFilename = RemoteBranch.Right(50);
		}
	}

	FString CommonAncestorFileId;	///< SHA1 Id of the file (warning: not the commit Id)
	FString RemoteFileId;		///< SHA1 Id of the file (warning: not the commit Id)

	FString CommonAncestorFilename;
	FString RemoteFilename;
};

/** Execute a command to get the details of a conflict */
static void RunGetConflictStatus(const FString& InPathToGitBinary, const FString& InRepositoryRoot, const FString& InFile, FGitSourceControlState& InOutFileState)
{
	TArray<FString> ErrorMessages;
	TArray<FString> Results;
	TArray<FString> Files;
	Files.Add(InFile);
	TArray<FString> Parameters;
	Parameters.Add(TEXT("--unmerged"));
	bool bResult = RunCommandInternal(TEXT("ls-files"), InPathToGitBinary, InRepositoryRoot, Parameters, Files, Results, ErrorMessages);
	if(bResult && Results.Num() == 3)
	{
		// Parse the unmerge status: extract the base revision (or the other branch?)
		FGitConflictStatusParser ConflictStatus(Results);
		InOutFileState.PendingResolveInfo.BaseFile = ConflictStatus.CommonAncestorFilename;
		InOutFileState.PendingResolveInfo.BaseRevision = ConflictStatus.CommonAncestorFileId;
		InOutFileState.PendingResolveInfo.RemoteFile = ConflictStatus.RemoteFilename;
		InOutFileState.PendingResolveInfo.RemoteRevision = ConflictStatus.RemoteFileId;
	}
}

/// Convert filename relative to the repository root to absolute path (inplace)
void AbsoluteFilenames(const FString& InRepositoryRoot, TArray<FString>& InFileNames)
{
	for(auto& FileName : InFileNames)
	{
		FileName = FPaths::ConvertRelativePathToFull(InRepositoryRoot, FileName);
	}
}

/** Run a 'git ls-files' command to get all files tracked by Git recursively in a directory.
 *
 * Called in case of a "directory status" (no file listed in the command) when using the "Submit to Revision Control" menu.
*/
static bool ListFilesInDirectoryRecurse(const FString& InPathToGitBinary, const FString& InRepositoryRoot, const FString& InDirectory, TArray<FString>& OutFiles)
{
	TArray<FString> ErrorMessages;
	TArray<FString> Directory;
	Directory.Add(InDirectory);
	const bool bResult = RunCommandInternal(TEXT("ls-files"), InPathToGitBinary, InRepositoryRoot, TArray<FString>(), Directory, OutFiles, ErrorMessages);
	AbsoluteFilenames(InRepositoryRoot, OutFiles);
	return bResult;
}

/** Parse the array of strings results of a 'git status' command for a provided list of files all in a common directory
 *
 * Called in case of a normal refresh of status on a list of assets in a the Content Browser (or user selected "Refresh" context menu).
 *
 * Example git status results:
M  Content/Textures/T_Perlin_Noise_M.uasset
R  Content/Textures/T_Perlin_Noise_M.uasset -> Content/Textures/T_Perlin_Noise_M2.uasset
?? Content/Materials/M_Basic_Wall.uasset
!! BasicCode.sln
*/
static void ParseFileStatusResult(const FString& InPathToGitBinary, const FString& InRepositoryRoot, const TArray<FString>& InFiles, const TArray<FString>& InResults, TArray<FGitSourceControlState>& OutStates)
{
	const FDateTime Now = FDateTime::Now();

	// Iterate on all files explicitly listed in the command
	for(const auto& File : InFiles)
	{
		FGitSourceControlState FileState(File);
		// Search the file in the list of status
		int32 IdxResult = InResults.IndexOfByPredicate(FGitStatusFileMatcher(File));
		if(IdxResult != INDEX_NONE)
		{
			// File found in status results; only the case for "changed" files
			FGitStatusParser StatusParser(InResults[IdxResult]);
			FileState.WorkingCopyState = StatusParser.State;
			if(FileState.IsConflicted())
			{
				// In case of a conflict (unmerged file) get the base revision to merge
				RunGetConflictStatus(InPathToGitBinary, InRepositoryRoot, File, FileState);
			}
		}
		else
		{
			// File not found in status
			if(FPaths::FileExists(File))
			{
				// usually means the file is unchanged,
				FileState.WorkingCopyState = EWorkingCopyState::Unchanged;
			}
			else
			{
				// but also the case for newly created content: there is no file on disk until the content is saved for the first time
				FileState.WorkingCopyState = EWorkingCopyState::NotControlled;
			}
		}
		FileState.TimeStamp = Now;
		OutStates.Add(FileState);
	}
}

/** Parse the array of strings results of a 'git status' command for a directory
 *
 *  Called in case of a "directory status" (no file listed in the command) ONLY to detect Deleted/Missing/Untracked files
 * since those files are not listed by the 'git ls-files' command.
 *
 * @see #ParseFileStatusResult() above for an example of a 'git status' results
*/
static void ParseDirectoryStatusResult(const FString& InPathToGitBinary, const FString& InRepositoryRoot, const TArray<FString>& InResults, TArray<FGitSourceControlState>& OutStates)
{
	// Iterate on each line of result of the status command
	for(const FString& Result : InResults)
	{
		const FString RelativeFilename = FilenameFromGitStatus(Result);
		const FString File = FPaths::ConvertRelativePathToFull(InRepositoryRoot, RelativeFilename);

		FGitSourceControlState FileState(File);
		FGitStatusParser StatusParser(Result);
		if((EWorkingCopyState::Deleted == StatusParser.State) || (EWorkingCopyState::Missing == StatusParser.State) || (EWorkingCopyState::NotControlled == StatusParser.State))
		{
			FileState.WorkingCopyState = StatusParser.State;
			FileState.TimeStamp.Now();
			OutStates.Add(MoveTemp(FileState));
		}
	}
}

/**
 * @brief Detects how to parse the result of a "status" command to get workspace file states
 *
 *  It is either a command for a whole directory (ie. "Content/", in case of "Submit to Revision Control" menu),
 * or for one or more files all on a same directory (by design, since we group files by directory in RunUpdateStatus())
 *
 * @param[in]	InPathToGitBinary	The path to the Git binary
 * @param[in]	InRepositoryRoot	The Git repository from where to run the command - usually the Game directory (can be empty)
 * @param[in]	InFiles				List of files in a directory, or the path to the directory itself (never empty).
 * @param[out]	InResults			Results from the "status" command
 * @param[out]	OutStates			States of files for witch the status has been gathered (distinct than InFiles in case of a "directory status")
 */
static void ParseStatusResults(const FString& InPathToGitBinary, const FString& InRepositoryRoot, const TArray<FString>& InFiles, const TArray<FString>& InResults, TArray<FGitSourceControlState>& OutStates)
{
	if(1 == InFiles.Num() && FPaths::DirectoryExists(InFiles[0]))
	{
		// 1) Special case for "status" of a directory: requires to get the list of files by ourselves.
		//   (this is triggered by the "Submit to Revision Control" menu)
		TArray<FString> Files;
		const FString& Directory = InFiles[0];
		const bool bResult = ListFilesInDirectoryRecurse(InPathToGitBinary, InRepositoryRoot, Directory, Files);
		if(bResult)
		{
			ParseFileStatusResult(InPathToGitBinary, InRepositoryRoot, Files, InResults, OutStates);
		}
		// The above cannot detect deleted assets since there is no file left to enumerate (either by the Content Browser or by git ls-files)
		// => so we also parse the status results to explicitly look for Deleted/Missing assets
		ParseDirectoryStatusResult(InPathToGitBinary, InRepositoryRoot, InResults, OutStates);
	}
	else
	{
		// 2) General case for one or more files in the same directory.
		ParseFileStatusResult(InPathToGitBinary, InRepositoryRoot, InFiles, InResults, OutStates);
	}
}

// Run a batch of Git "status" command to update status of given files and/or directories.
bool RunUpdateStatus(const FString& InPathToGitBinary, const FString& InRepositoryRoot, const TArray<FString>& InFiles, TArray<FString>& OutErrorMessages, TArray<FGitSourceControlState>& OutStates)
{
	bool bResults = true;
	TArray<FString> Results;
	TArray<FString> Parameters;
	Parameters.Add(TEXT("--porcelain"));
	Parameters.Add(TEXT("--ignored"));

	// Git status does not show any "untracked files" when called with files from different subdirectories! (issue #3)
	// 1) So here we group files by path (ie. by subdirectory)
	TMap<FString, TArray<FString>> GroupOfFiles;
	for(const auto& File : InFiles)
	{
		const FString Path = FPaths::GetPath(*File);
		TArray<FString>* Group = GroupOfFiles.Find(Path);
		if(Group != nullptr)
		{
			Group->Add(File);
		}
		else
		{
			TArray<FString> NewGroup;
			NewGroup.Add(File);
			GroupOfFiles.Add(Path, NewGroup);
		}
	}

	// 2) then we can batch git status operation by subdirectory
	for(const auto& Files : GroupOfFiles)
	{
		// "git status" can only detect renamed and deleted files when it operate on a folder, so use one folder path for all files in a directory
		const FString Path = FPaths::GetPath(*Files.Value[0]);
		TArray<FString> OnePath;
		// Only one file: optim very useful for the .uproject file at the root to avoid parsing the whole repository
		// (works only if the file exists)
		if((1 == Files.Value.Num()) && (FPaths::FileExists(Files.Value[0])))
		{
			OnePath.Add(Files.Value[0]);
		}
		else
		{
			OnePath.Add(Path);
		}
		TArray<FString> ErrorMessages;
		const bool bResult = RunCommand(TEXT("status"), InPathToGitBinary, InRepositoryRoot, Parameters, OnePath, Results, ErrorMessages);
		OutErrorMessages.Append(ErrorMessages);
		if(bResult)
		{
			ParseStatusResults(InPathToGitBinary, InRepositoryRoot, Files.Value, Results, OutStates);
		}
	}

	return bResults;
}

// Run a Git `cat-file --filters` command to dump the binary content of a revision into a file.
bool RunDumpToFile(const FString& InPathToGitBinary, const FString& InRepositoryRoot, const FString& InParameter, const FString& InDumpFileName)
{
	int32 ReturnCode = -1;
	FString FullCommand;

	FGitSourceControlModule& GitSourceControl = FModuleManager::LoadModuleChecked<FGitSourceControlModule>("GitSourceControl");
	const FGitVersion& GitVersion = GitSourceControl.GetProvider().GetGitVersion();

	if(!InRepositoryRoot.IsEmpty())
	{
		// Specify the working copy (the root) of the git repository (before the command itself)
		FullCommand  = TEXT("-C \"");
		FullCommand += InRepositoryRoot;
		FullCommand += TEXT("\" ");
	}

	// then the git command itself
	if(GitVersion.bHasCatFileWithFilters)
	{
		// Newer versions (2.9.3.windows.2) support smudge/clean filters used by Git LFS, git-fat, git-annex, etc
		FullCommand += TEXT("cat-file --filters ");
	}
	else
	{
		// Previous versions fall-back on "git show" like before
		FullCommand += TEXT("show ");
	}

	// Append to the command the parameter
	FullCommand += InParameter;

	const bool bLaunchDetached = false;
	const bool bLaunchHidden = true;
	const bool bLaunchReallyHidden = bLaunchHidden;

	void* PipeRead = nullptr;
	void* PipeWrite = nullptr;

	verify(FPlatformProcess::CreatePipe(PipeRead, PipeWrite));

	FProcHandle ProcessHandle = FPlatformProcess::CreateProc(*InPathToGitBinary, *FullCommand, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, nullptr, 0, *InRepositoryRoot, PipeWrite, nullptr, nullptr);
	if(ProcessHandle.IsValid())
	{
		FPlatformProcess::Sleep(0.01f);

		TArray<uint8> BinaryFileContent;
		while(FPlatformProcess::IsProcRunning(ProcessHandle))
		{
			TArray<uint8> BinaryData;
			if(FPlatformProcess::ReadPipeToArray(PipeRead, BinaryData))
			{
				BinaryFileContent.Append(MoveTemp(BinaryData));
			}
		}
		TArray<uint8> BinaryData;
		while(FPlatformProcess::ReadPipeToArray(PipeRead, BinaryData))
		{
			BinaryFileContent.Append(MoveTemp(BinaryData));
		}

		FPlatformProcess::GetProcReturnCode(ProcessHandle, &ReturnCode);
		if(ReturnCode == 0)
		{
			// Save buffer into temp file
			if(FFileHelper::SaveArrayToFile(BinaryFileContent, *InDumpFileName))
			{
				UE_LOGF(LogSourceControl, Log, "Writed '%ls' (%do)", *InDumpFileName, BinaryFileContent.Num());
			}
			else
			{
				UE_LOGF(LogSourceControl, Error, "Could not write %ls", *InDumpFileName);
				ReturnCode = -1;
			}
		}
		else
		{
			UE_LOGF(LogSourceControl, Error, "DumpToFile: ReturnCode=%d", ReturnCode);
		}

		FPlatformProcess::CloseProc(ProcessHandle);
	}
	else
	{
		UE_LOGF(LogSourceControl, Error, "Failed to launch 'git cat-file'");
	}

	FPlatformProcess::ClosePipe(PipeRead, PipeWrite);

	return (ReturnCode == 0);
}

/**
 * Translate file actions from the given Git log --name-status command to keywords used by the Editor UI.
 *
 * @see https://www.kernel.org/pub/software/scm/git/docs/git-log.html
 * ' ' = unmodified
 * 'M' = modified
 * 'A' = added
 * 'D' = deleted
 * 'R' = renamed
 * 'C' = copied
 * 'T' = type changed
 * 'U' = updated but unmerged
 * 'X' = unknown
 * 'B' = broken pairing
 *
 * @see SHistoryRevisionListRowContent::GenerateWidgetForColumn(): "add", "edit", "delete", "branch" and "integrate" (everything else is taken like "edit")
*/
static FString LogStatusToString(TCHAR InStatus)
{
	switch(InStatus)
	{
	case TEXT(' '):
		return FString("unmodified");
	case TEXT('M'):
		return FString("modified");
	case TEXT('A'): // added: keyword "add" to display a specific icon instead of the default "edit" action one
		return FString("add");
	case TEXT('D'): // deleted: keyword "delete" to display a specific icon instead of the default "edit" action one
		return FString("delete");
	case TEXT('R'): // renamed keyword "branch" to display a specific icon instead of the default "edit" action one
		return FString("branch");
	case TEXT('C'): // copied keyword "branch" to display a specific icon instead of the default "edit" action one
		return FString("branch");
	case TEXT('T'):
		return FString("type changed");
	case TEXT('U'):
		return FString("unmerged");
	case TEXT('X'):
		return FString("unknown");
	case TEXT('B'):
		return FString("broked pairing");
	}

	return FString();
}

/**
 * Parse the array of strings results of a 'git log' command
 *
 * Example git log results:
commit 97a4e7626681895e073aaefd68b8ac087db81b0b
Author: Sébastien Rombauts <sebastien.rombauts@gmail.com>
Date:   2014-2015-05-15 21:32:27 +0200

    Another commit used to test History

     - with many lines
     - some <xml>
     - and strange characteres $*+

M	Content/Blueprints/Blueprint_CeilingLight.uasset
R100	Content/Textures/T_Concrete_Poured_D.uasset Content/Textures/T_Concrete_Poured_D2.uasset

commit 355f0df26ebd3888adbb558fd42bb8bd3e565000
Author: Sébastien Rombauts <sebastien.rombauts@gmail.com>
Date:   2014-2015-05-12 11:28:14 +0200

    Testing git status, edit, and revert

A	Content/Blueprints/Blueprint_CeilingLight.uasset
C099	Content/Textures/T_Concrete_Poured_N.uasset Content/Textures/T_Concrete_Poured_N2.uasset
*/
static void ParseLogResults(const TArray<FString>& InResults, TGitSourceControlHistory& OutHistory)
{
	TSharedRef<FGitSourceControlRevision, ESPMode::ThreadSafe> SourceControlRevision = MakeShareable(new FGitSourceControlRevision);
	for(const auto& Result : InResults)
	{
		if(Result.StartsWith(TEXT("commit "))) // Start of a new commit
		{
			// End of the previous commit
			if(SourceControlRevision->RevisionNumber != 0)
			{
				OutHistory.Add(MoveTemp(SourceControlRevision));

				SourceControlRevision = MakeShareable(new FGitSourceControlRevision);
			}
			SourceControlRevision->CommitId = Result.RightChop(7); // Full commit SHA1 hexadecimal string
			SourceControlRevision->ShortCommitId = SourceControlRevision->CommitId.Left(8); // Short revision ; first 8 hex characters (max that can hold a 32 bit integer)
			SourceControlRevision->CommitIdNumber = FParse::HexNumber(*SourceControlRevision->ShortCommitId);
			SourceControlRevision->RevisionNumber = -1; // RevisionNumber will be set at the end, based off the index in the History
		}
		else if(Result.StartsWith(TEXT("Author: "))) // Author name & email
		{
			// Remove the 'email' part of the UserName
			FString UserNameEmail = Result.RightChop(8);
			int32 EmailIndex = 0;
			if(UserNameEmail.FindLastChar('<', EmailIndex))
			{
				SourceControlRevision->UserName = UserNameEmail.Left(EmailIndex - 1);
			}
		}
		else if(Result.StartsWith(TEXT("Date:   "))) // Commit date
		{
			FString Date = Result.RightChop(8);
			SourceControlRevision->Date = FDateTime::FromUnixTimestamp(FCString::Atoi(*Date));
		}
	//	else if(Result.IsEmpty()) // empty line before/after commit message has already been taken care by FString::ParseIntoArray()
		else if(Result.StartsWith(TEXT("    ")))  // Multi-lines commit message
		{
			SourceControlRevision->Description += Result.RightChop(4);
			SourceControlRevision->Description += TEXT("\n");
		}
		else // Name of the file, starting with an uppercase status letter ("A"/"M"...)
		{
			const TCHAR Status = Result[0];
			SourceControlRevision->Action = LogStatusToString(Status); // Readable action string ("Added", Modified"...) instead of "A"/"M"...
			// Take care of special case for Renamed/Copied file: extract the second filename after second tabulation
			int32 IdxTab;
			if(Result.FindLastChar('\t', IdxTab))
			{
				SourceControlRevision->Filename = Result.RightChop(IdxTab + 1); // relative filename
			}
		}
	}
	// End of the last commit
	if(SourceControlRevision->RevisionNumber != 0)
	{
		OutHistory.Add(MoveTemp(SourceControlRevision));
	}

	// Then set the revision number of each Revision based on its index (reverse order since the log starts with the most recent change)
	for(int32 RevisionIndex = 0; RevisionIndex < OutHistory.Num(); RevisionIndex++)
	{
		const auto& SourceControlRevisionItem = OutHistory[RevisionIndex];
		SourceControlRevisionItem->RevisionNumber = OutHistory.Num() - RevisionIndex;

		// Special case of a move ("branch" in Perforce term): point to the previous change (so the next one in the order of the log)
		if((SourceControlRevisionItem->Action == "branch") && (RevisionIndex < OutHistory.Num() - 1))
		{
			SourceControlRevisionItem->BranchSource = OutHistory[RevisionIndex + 1];
		}
	}
}

/**
 * Extract the SHA1 identifier and size of a blob (file) from a Git "ls-tree" command.
 *
 * Example output for the command git ls-tree --long 7fdaeb2 Content/Blueprints/BP_Test.uasset
100644 blob a14347dc3b589b78fb19ba62a7e3982f343718bc   70731	Content/Blueprints/BP_Test.uasset
*/
class FGitLsTreeParser
{
public:
	/** Parse the unmerge status: extract the base SHA1 identifier of the file */
	FGitLsTreeParser(const TArray<FString>& InResults)
	{
		const FString& FirstResult = InResults[0];
		FileHash = FirstResult.Mid(12, 40);
		int32 IdxTab;
		if(FirstResult.FindChar('\t', IdxTab))
		{
			const FString SizeString = FirstResult.Mid(53, IdxTab - 53);
			FileSize = FCString::Atoi(*SizeString);
		}
	}

	FString FileHash;	///< SHA1 Id of the file (warning: not the commit Id)
	int32	FileSize;	///< Size of the file (in bytes)
};

// Run a Git "log" command and parse it.
bool RunGetHistory(const FString& InPathToGitBinary, const FString& InRepositoryRoot, const FString& InFile, bool bMergeConflict, TArray<FString>& OutErrorMessages, TGitSourceControlHistory& OutHistory)
{
	bool bResults;
	{
		TArray<FString> Results;
		TArray<FString> Parameters;
		Parameters.Add(TEXT("--follow")); // follow file renames
		Parameters.Add(TEXT("--date=raw"));
		Parameters.Add(TEXT("--name-status")); // relative filename at this revision, preceded by a status character
		Parameters.Add(TEXT("--pretty=medium")); // make sure format matches expected in ParseLogResults
		if(bMergeConflict)
		{
			// In case of a merge conflict, we also need to get the tip of the "remote branch" (MERGE_HEAD) before the log of the "current branch" (HEAD)
			// @todo does not work for a cherry-pick! Test for a rebase.
			Parameters.Add(TEXT("MERGE_HEAD"));
			Parameters.Add(TEXT("--max-count 1"));
		}
		TArray<FString> Files;
		Files.Add(*InFile);
		bResults = RunCommand(TEXT("log"), InPathToGitBinary, InRepositoryRoot, Parameters, Files, Results, OutErrorMessages);
		if(bResults)
		{
			ParseLogResults(Results, OutHistory);
		}
	}
	for(auto& Revision : OutHistory)
	{
		// Get file (blob) sha1 id and size
		TArray<FString> Results;
		TArray<FString> Parameters;
		Parameters.Add(TEXT("--long")); // Show object size of blob (file) entries.
		Parameters.Add(Revision->GetRevision());
		TArray<FString> Files;
		Files.Add(*Revision->GetFilename());
		bResults &= RunCommand(TEXT("ls-tree"), InPathToGitBinary, InRepositoryRoot, Parameters, Files, Results, OutErrorMessages);
		if(bResults && Results.Num())
		{
			FGitLsTreeParser LsTree(Results);
			Revision->FileHash = LsTree.FileHash;
			Revision->FileSize = LsTree.FileSize;
		}
	}

	return bResults;
}

bool UpdateCachedStates(const TArray<FGitSourceControlState>& InStates)
{
	FGitSourceControlModule& GitSourceControl = FModuleManager::LoadModuleChecked<FGitSourceControlModule>( "GitSourceControl" );
	FGitSourceControlProvider& Provider = GitSourceControl.GetProvider();
	int NbStatesUpdated = 0;

	for(const auto& InState : InStates)
	{
		TSharedRef<FGitSourceControlState, ESPMode::ThreadSafe> State = Provider.GetStateInternal(InState.LocalFilename);
		if(State->WorkingCopyState != InState.WorkingCopyState)
		{
			State->WorkingCopyState = InState.WorkingCopyState;
			State->PendingResolveInfo = InState.PendingResolveInfo;
		//	State->TimeStamp = InState.TimeStamp; // @todo Bug report: Workaround a bug with the Source Control Module not updating file state after a "Save"
			NbStatesUpdated++;
		}
	}

	return (NbStatesUpdated > 0);
}

/**
 * Helper struct for RemoveRedundantErrors()
 */
struct FRemoveRedundantErrors
{
	FRemoveRedundantErrors(const FString& InFilter)
		: Filter(InFilter)
	{
	}

	bool operator()(const FString& String) const
	{
		if(String.Contains(Filter))
		{
			return true;
		}

		return false;
	}

	/** The filter string we try to identify in the reported error */
	FString Filter;
};

void RemoveRedundantErrors(FGitSourceControlCommand& InCommand, const FString& InFilter)
{
	bool bFoundRedundantError = false;
	for(auto Iter(InCommand.ErrorMessages.CreateConstIterator()); Iter; Iter++)
	{
		if(Iter->Contains(InFilter))
		{
			InCommand.InfoMessages.Add(*Iter);
			bFoundRedundantError = true;
		}
	}

	InCommand.ErrorMessages.RemoveAll( FRemoveRedundantErrors(InFilter) );

	// if we have no error messages now, assume success!
	if(bFoundRedundantError && InCommand.ErrorMessages.Num() == 0 && !InCommand.bCommandSuccessful)
	{
		InCommand.bCommandSuccessful = true;
	}
}

}
