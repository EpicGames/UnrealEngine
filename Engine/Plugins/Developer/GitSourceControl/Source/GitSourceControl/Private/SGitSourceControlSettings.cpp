// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGitSourceControlSettings.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SFilePathPicker.h"
#include "EditorDirectories.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "SourceControlOperations.h"
#include "GitSourceControlModule.h"
#include "GitSourceControlUtils.h"

#define LOCTEXT_NAMESPACE "SGitSourceControlSettings"

void SGitSourceControlSettings::Construct(const FArguments& InArgs)
{
	bAutoCreateGitIgnore = true;
	bAutoCreateReadme = true;
	bAutoCreateGitAttributes = false;
	bAutoInitialCommit = true;

	InitialCommitMessage = LOCTEXT("InitialCommitMessage", "Initial commit");

	const FText FileFilterType = NSLOCTEXT("GitSourceControl", "Executables", "Executables");
#if PLATFORM_WINDOWS
	const FString FileFilterText = FString::Printf(TEXT("%s (*.exe)|*.exe"), *FileFilterType.ToString());
#else
	const FString FileFilterText = FString::Printf(TEXT("%s"), *FileFilterType.ToString());
#endif

	ReadmeContent = FText::FromString(FString(TEXT("# ")) + FApp::GetProjectName() + "\n\nDeveloped with Unreal Engine 5\n");

	ChildSlot
	[
		SNew(SVerticalBox)
		// Path to the Git command line executable
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("BinaryPathLabel", "Git Path"))
				.ToolTipText(LOCTEXT("BinaryPathLabel_Tooltip", "Path to Git binary"))
			]
			+SHorizontalBox::Slot()
			.FillWidth(2.0f)
			[
				SNew(SFilePathPicker)
				.BrowseButtonImage(FAppStyle::GetBrush("PropertyWindow.Button_Ellipsis"))
				.BrowseButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.BrowseButtonToolTip(LOCTEXT("BinaryPathLabel_Tooltip", "Path to Git binary"))
				.BrowseDirectory(FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_OPEN))
				.BrowseTitle(LOCTEXT("BinaryPathBrowseTitle", "File picker..."))
				.FilePath(this, &SGitSourceControlSettings::GetBinaryPathString)
				.FileTypeFilter(FileFilterText)
				.OnPathPicked(this, &SGitSourceControlSettings::OnBinaryPathPicked)
			]
		]
		// Root of the local repository
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("RepositoryRootLabel", "Root of the repository"))
				.ToolTipText(LOCTEXT("RepositoryRootLabel_Tooltip", "Path to the root of the Git repository"))
			]
			+SHorizontalBox::Slot()
			.FillWidth(2.0f)
			[
				SNew(STextBlock)
				.Text(this, &SGitSourceControlSettings::GetPathToRepositoryRoot)
				.ToolTipText(LOCTEXT("RepositoryRootLabel_Tooltip", "Path to the root of the Git repository"))
			]
		]
		// User Name
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("GitUserName", "User Name"))
				.ToolTipText(LOCTEXT("GitUserName_Tooltip", "User name configured for the Git repository"))
			]
			+SHorizontalBox::Slot()
			.FillWidth(2.0f)
			[
				SNew(STextBlock)
				.Text(this, &SGitSourceControlSettings::GetUserName)
				.ToolTipText(LOCTEXT("GitUserName_Tooltip", "User name configured for the Git repository"))
			]
		]
		// User e-mail
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("GitUserEmail", "E-Mail"))
				.ToolTipText(LOCTEXT("GitUserEmail_Tooltip", "User e-mail configured for the Git repository"))
			]
			+SHorizontalBox::Slot()
			.FillWidth(2.0f)
			[
				SNew(STextBlock)
				.Text(this, &SGitSourceControlSettings::GetUserEmail)
				.ToolTipText(LOCTEXT("GitUserEmail_Tooltip", "User e-mail configured for the Git repository"))
			]
		]
		// Separator
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SSeparator)
		]
		// Explanation text
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SGitSourceControlSettings::CanInitializeGitRepository)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("RepositoryNotFound", "Current Project is not contained in a Git Repository. Fill the form below to initialize a new Repository."))
				.WrapTextAt(450.0f)
				.ToolTipText(LOCTEXT("RepositoryNotFound_Tooltip", "No Repository found at the level or above the current Project"))
			]
		]
		// Option to configure the URL of the default remote 'origin'
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SGitSourceControlSettings::CanInitializeGitRepository)
			.ToolTipText(LOCTEXT("ConfigureOrigin_Tooltip", "Configure the URL of the default remote 'origin'"))
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ConfigureOrigin", "URL of the remote server 'origin'"))
			]
			+SHorizontalBox::Slot()
			.FillWidth(2.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SEditableTextBox)
				.Text(this, &SGitSourceControlSettings::GetRemoteUrl)
				.OnTextCommitted(this, &SGitSourceControlSettings::OnRemoteUrlCommited)
			]
		]
		// Option to add a proper .gitignore file (true by default)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SGitSourceControlSettings::CanInitializeGitRepository)
			+SHorizontalBox::Slot()
			[
				SNew(SCheckBox)
				.ForegroundColor(FSlateColor::UseForeground())
				.ToolTipText(LOCTEXT("CreateGitIgnore_Tooltip", "Create and add a standard '.gitignore' file"))
				.IsChecked(ECheckBoxState::Checked)
				.OnCheckStateChanged(this, &SGitSourceControlSettings::OnCheckedCreateGitIgnore)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("CreateGitIgnore", "Add a .gitignore file"))
					.ToolTipText(LOCTEXT("CreateGitIgnore_Tooltip", "Create and add a standard '.gitignore' file"))
				]
			]
		]
		// Option to add a README.md file with custom content
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SGitSourceControlSettings::CanInitializeGitRepository)
			.ToolTipText(LOCTEXT("CreateReadme_Tooltip", "Add a README.md file"))
			+SHorizontalBox::Slot()
			[
				SNew(SCheckBox)
				.ForegroundColor(FSlateColor::UseForeground())
				.IsChecked(ECheckBoxState::Checked)
				.OnCheckStateChanged(this, &SGitSourceControlSettings::OnCheckedCreateReadme)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("CreateReadme", "Add a basic README.md file"))
				]
			]
			+SHorizontalBox::Slot()
			.Padding(2.0f)
			[
				SNew(SMultiLineEditableTextBox)
				.Text(this, &SGitSourceControlSettings::GetReadmeContent)
				.OnTextCommitted(this, &SGitSourceControlSettings::OnReadmeContentCommited)
				.IsEnabled(this, &SGitSourceControlSettings::GetAutoCreateReadme)
				.SelectAllTextWhenFocused(true)
			]
		]
		// Option to add a proper .gitattributes file for Git LFS (false by default)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SGitSourceControlSettings::CanInitializeGitRepository)
			+SHorizontalBox::Slot()
			[
				SNew(SCheckBox)
				.ForegroundColor(FSlateColor::UseForeground())
				.ToolTipText(LOCTEXT("CreateGitAttributesAll_Tooltip", "Create and add a '.gitattributes' file to enable Git LFS for the whole 'Content/' directory (needs Git LFS extensions to be installed)."))
				.IsChecked(ECheckBoxState::Unchecked)
				.OnCheckStateChanged(this, &SGitSourceControlSettings::OnCheckedCreateGitAttributes)
				.IsEnabled(this, &SGitSourceControlSettings::CanInitializeGitLfs)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("CreateGitAttributes", "Add a .gitattributes file to enable Git LFS"))
				]
			]
		]
		// Option to Make the initial Git commit with custom message
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SGitSourceControlSettings::CanInitializeGitRepository)
			+SHorizontalBox::Slot()
			[
				SNew(SCheckBox)
				.ForegroundColor(FSlateColor::UseForeground())
				.ToolTipText(LOCTEXT("InitialGitCommit_Tooltip", "Make the initial Git commit"))
				.IsChecked(ECheckBoxState::Checked)
				.OnCheckStateChanged(this, &SGitSourceControlSettings::OnCheckedInitialCommit)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("InitialGitCommit", "Make the initial Git Commit"))
				]
			]
			+SHorizontalBox::Slot()
			.Padding(2.0f)
			[
				SNew(SMultiLineEditableTextBox)
				.Text(this, &SGitSourceControlSettings::GetInitialCommitMessage)
				.ToolTipText(LOCTEXT("InitialCommitMessage_Tooltip", "Message of initial commit"))
				.OnTextCommitted(this, &SGitSourceControlSettings::OnInitialCommitMessageCommited)
			]
		]
		// Button to initialize the project with Git, create .gitignore/.gitattributes files, and make the first commit)
		+SVerticalBox::Slot()
		.FillHeight(2.5f)
		.Padding(4.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SGitSourceControlSettings::CanInitializeGitRepository)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SButton)
				.Text(LOCTEXT("GitInitRepository", "Initialize project with Git"))
				.ToolTipText(LOCTEXT("GitInitRepository_Tooltip", "Initialize current project as a new Git repository"))
				.OnClicked(this, &SGitSourceControlSettings::OnClickedInitializeGitRepository)
				.HAlign(HAlign_Center)
				.ContentPadding(6.0f)
			]
		]
	];
}

SGitSourceControlSettings::~SGitSourceControlSettings()
{
	RemoveInProgressNotification();
}

FString SGitSourceControlSettings::GetBinaryPathString() const
{
	FGitSourceControlModule& GitSourceControl = FModuleManager::LoadModuleChecked<FGitSourceControlModule>("GitSourceControl");
	return GitSourceControl.AccessSettings().GetBinaryPath();
}

void SGitSourceControlSettings::OnBinaryPathPicked( const FString& PickedPath ) const
{
	FGitSourceControlModule& GitSourceControl = FModuleManager::LoadModuleChecked<FGitSourceControlModule>("GitSourceControl");
	FString PickedFullPath = FPaths::ConvertRelativePathToFull(PickedPath);
	const bool bChanged = GitSourceControl.AccessSettings().SetBinaryPath(PickedFullPath);
	if(bChanged)
	{
		// Re-Check provided git binary path for each change
		GitSourceControl.GetProvider().CheckGitAvailability();
		if(GitSourceControl.GetProvider().IsGitAvailable())
		{
			GitSourceControl.SaveSettings();
		}
	}
}

FText SGitSourceControlSettings::GetPathToRepositoryRoot() const
{
	FGitSourceControlModule& GitSourceControl = FModuleManager::LoadModuleChecked<FGitSourceControlModule>("GitSourceControl");
	return FText::FromString(GitSourceControl.GetProvider().GetPathToRepositoryRoot());
}

FText SGitSourceControlSettings::GetUserName() const
{
	FGitSourceControlModule& GitSourceControl = FModuleManager::LoadModuleChecked<FGitSourceControlModule>("GitSourceControl");
	return FText::FromString(GitSourceControl.GetProvider().GetUserName());
}

FText SGitSourceControlSettings::GetUserEmail() const
{
	FGitSourceControlModule& GitSourceControl = FModuleManager::LoadModuleChecked<FGitSourceControlModule>("GitSourceControl");
	return FText::FromString(GitSourceControl.GetProvider().GetUserEmail());
}

EVisibility SGitSourceControlSettings::CanInitializeGitRepository() const
{
	FGitSourceControlModule& GitSourceControl = FModuleManager::LoadModuleChecked<FGitSourceControlModule>("GitSourceControl");
	const bool bGitAvailable = GitSourceControl.GetProvider().IsGitAvailable();
	const bool bGitRepositoryFound = GitSourceControl.GetProvider().IsEnabled();
	return (bGitAvailable && !bGitRepositoryFound) ? EVisibility::Visible : EVisibility::Collapsed;
}

bool SGitSourceControlSettings::CanInitializeGitLfs() const
{
	FGitSourceControlModule& GitSourceControl = FModuleManager::LoadModuleChecked<FGitSourceControlModule>("GitSourceControl");
	const FString& PathToGitBinary = GitSourceControl.AccessSettings().GetBinaryPath();
	const bool bGitLfsAvailable = GitSourceControl.GetProvider().GetGitVersion().bHasGitLfs;
	const bool bGitRepositoryFound = GitSourceControl.GetProvider().IsEnabled();
	return (bGitLfsAvailable && !bGitRepositoryFound);
}

FReply SGitSourceControlSettings::OnClickedInitializeGitRepository()
{
	FGitSourceControlModule& GitSourceControl = FModuleManager::LoadModuleChecked<FGitSourceControlModule>("GitSourceControl");
	const FString& PathToGitBinary = GitSourceControl.AccessSettings().GetBinaryPath();
	const FString PathToProjectDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());
	TArray<FString> InfoMessages;
	TArray<FString> ErrorMessages;

	// 1.a. Synchronous (very quick) "git init" operation: initialize a Git local repository with a .git/ subdirectory
	GitSourceControlUtils::RunCommand(TEXT("init"), PathToGitBinary, PathToProjectDir, TArray<FString>(), TArray<FString>(), InfoMessages, ErrorMessages);
	// 1.b. Synchronous (very quick) "git remote add" operation: configure the URL of the default remote server 'origin' if specified
	if(!RemoteUrl.IsEmpty())
	{
		TArray<FString> Parameters;
		Parameters.Add(TEXT("add origin"));
		Parameters.Add(RemoteUrl.ToString());
		GitSourceControlUtils::RunCommand(TEXT("remote"), PathToGitBinary, PathToProjectDir, Parameters, TArray<FString>(), InfoMessages, ErrorMessages);
	}

	// Check the new repository status to enable connection (branch, user e-mail)
	GitSourceControl.GetProvider().CheckRepositoryStatus(PathToGitBinary);
	if(GitSourceControl.GetProvider().IsAvailable())
	{
		// List of files to add to Source Control (.uproject, Config/, Content/, Source/ files and .gitignore/.gitattributes if any)
		TArray<FString> ProjectFiles;
		ProjectFiles.Add(FPaths::GetProjectFilePath());
		ProjectFiles.Add(FPaths::ProjectConfigDir());
		ProjectFiles.Add(FPaths::ProjectContentDir());
		if (FPaths::DirectoryExists(FPaths::GameSourceDir()))
		{
			ProjectFiles.Add(FPaths::GameSourceDir());
		}
		if(bAutoCreateGitIgnore)
		{
			// 2.a. Create a standard ".gitignore" file with common patterns for a typical Blueprint & C++ project
			const FString GitIgnoreFilename = FPaths::Combine(FPaths::ProjectDir(), TEXT(".gitignore"));
			const FString GitIgnoreContent = TEXT("Binaries\nDerivedDataCache\nIntermediate\nSaved\n.vscode\n.vs\n*.VC.db\n*.opensdf\n*.opendb\n*.sdf\n*.sln\n*.suo\n*.xcodeproj\n*.xcworkspace");
			if(FFileHelper::SaveStringToFile(GitIgnoreContent, *GitIgnoreFilename, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM))
			{
				ProjectFiles.Add(GitIgnoreFilename);
			}
		}
		if(bAutoCreateReadme)
		{
			// 2.b. Create a "README.md" file with a custom description
			const FString ReadmeFilename = FPaths::Combine(FPaths::ProjectDir(), TEXT("README.md"));
			if (FFileHelper::SaveStringToFile(ReadmeContent.ToString(), *ReadmeFilename, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM))
			{
				ProjectFiles.Add(ReadmeFilename);
			}
		}
		if(bAutoCreateGitAttributes)
		{
			// 2.c. Synchronous (very quick) "lfs install" operation: needs only to be run once by user
			GitSourceControlUtils::RunCommand(TEXT("lfs install"), PathToGitBinary, PathToProjectDir, TArray<FString>(), TArray<FString>(), InfoMessages, ErrorMessages);

			// 2.d. Create a ".gitattributes" file to enable Git LFS (Large File System) for the whole "Content/" subdir
			const FString GitAttributesFilename = FPaths::Combine(FPaths::ProjectDir(), TEXT(".gitattributes"));
			const FString GitAttributesContent = TEXT("Content/** filter=lfs diff=lfs merge=lfs -text\n");
			if (FFileHelper::SaveStringToFile(GitAttributesContent, *GitAttributesFilename, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM))
			{
				ProjectFiles.Add(GitAttributesFilename);
			}
		}

		// 3. Add files to Source Control: launch an asynchronous MarkForAdd operation
		LaunchMarkForAddOperation(ProjectFiles);

		// 4. The CheckIn will follow, at completion of the MarkForAdd operation
	}
	return FReply::Handled();
}

// Launch an asynchronous "MarkForAdd" operation and start an ongoing notification
void SGitSourceControlSettings::LaunchMarkForAddOperation(const TArray<FString>& InFiles)
{
	FGitSourceControlModule& GitSourceControl = FModuleManager::LoadModuleChecked<FGitSourceControlModule>("GitSourceControl");
	TSharedRef<FMarkForAdd, ESPMode::ThreadSafe> MarkForAddOperation = ISourceControlOperation::Create<FMarkForAdd>();
	ECommandResult::Type Result = GitSourceControl.GetProvider().Execute(MarkForAddOperation, InFiles, EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateSP(this, &SGitSourceControlSettings::OnSourceControlOperationComplete));
	if (Result == ECommandResult::Succeeded)
	{
		DisplayInProgressNotification(MarkForAddOperation);
	}
	else
	{
		DisplayFailureNotification(MarkForAddOperation);
	}
}

// Launch an asynchronous "CheckIn" operation and start another ongoing notification
void SGitSourceControlSettings::LaunchCheckInOperation()
{
	TSharedRef<FCheckIn, ESPMode::ThreadSafe> CheckInOperation = ISourceControlOperation::Create<FCheckIn>();
	CheckInOperation->SetDescription(InitialCommitMessage);
	FGitSourceControlModule& GitSourceControl = FModuleManager::LoadModuleChecked<FGitSourceControlModule>("GitSourceControl");
	ECommandResult::Type Result = GitSourceControl.GetProvider().Execute(CheckInOperation, TArray<FString>(), EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateSP(this, &SGitSourceControlSettings::OnSourceControlOperationComplete));
	if (Result == ECommandResult::Succeeded)
	{
		DisplayInProgressNotification(CheckInOperation);
	}
	else
	{
		DisplayFailureNotification(CheckInOperation);
	}
}

/// Delegate called when a source control operation has completed: launch the next one and manage notifications
void SGitSourceControlSettings::OnSourceControlOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	RemoveInProgressNotification();

	// Report result with a notification
	if (InResult == ECommandResult::Succeeded)
	{
		DisplaySuccessNotification(InOperation);
	}
	else
	{
		DisplayFailureNotification(InOperation);
	}

	if ((InOperation->GetName() == "MarkForAdd") && (InResult == ECommandResult::Succeeded) && bAutoInitialCommit)
	{
		// 4. optional initial Asynchronous commit with custom message: launch a "CheckIn" Operation
		LaunchCheckInOperation();
	}
}


// Display an ongoing notification during the whole operation
void SGitSourceControlSettings::DisplayInProgressNotification(const FSourceControlOperationRef& InOperation)
{
	FNotificationInfo Info(InOperation->GetInProgressString());
	Info.bFireAndForget = false;
	Info.ExpireDuration = 0.0f;
	Info.FadeOutDuration = 1.0f;
	OperationInProgressNotification = FSlateNotificationManager::Get().AddNotification(Info);
	if (OperationInProgressNotification.IsValid())
	{
		OperationInProgressNotification.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
	}
}

// Remove the ongoing notification at the end of the operation
void SGitSourceControlSettings::RemoveInProgressNotification()
{
	if (OperationInProgressNotification.IsValid())
	{
		OperationInProgressNotification.Pin()->ExpireAndFadeout();
		OperationInProgressNotification.Reset();
	}
}

// Display a temporary success notification at the end of the operation
void SGitSourceControlSettings::DisplaySuccessNotification(const FSourceControlOperationRef& InOperation)
{
	const FText NotificationText = FText::Format(LOCTEXT("InitialCommit_Success", "{0} operation was successfull!"), FText::FromName(InOperation->GetName()));
	FNotificationInfo Info(NotificationText);
	Info.bUseSuccessFailIcons = true;
	Info.Image = FAppStyle::GetBrush(TEXT("NotificationList.SuccessImage"));
	FSlateNotificationManager::Get().AddNotification(Info);
}

// Display a temporary failure notification at the end of the operation
void SGitSourceControlSettings::DisplayFailureNotification(const FSourceControlOperationRef& InOperation)
{
	const FText NotificationText = FText::Format(LOCTEXT("InitialCommit_Failure", "Error: {0} operation failed!"), FText::FromName(InOperation->GetName()));
	FNotificationInfo Info(NotificationText);
	Info.ExpireDuration = 8.0f;
	FSlateNotificationManager::Get().AddNotification(Info);
}

void SGitSourceControlSettings::OnCheckedCreateGitIgnore(ECheckBoxState NewCheckedState)
{
	bAutoCreateGitIgnore = (NewCheckedState == ECheckBoxState::Checked);
}

void SGitSourceControlSettings::OnCheckedCreateReadme(ECheckBoxState NewCheckedState)
{
	bAutoCreateReadme = (NewCheckedState == ECheckBoxState::Checked);
}

bool SGitSourceControlSettings::GetAutoCreateReadme() const
{
	return bAutoCreateReadme;
}

void SGitSourceControlSettings::OnReadmeContentCommited(const FText& InText, ETextCommit::Type InCommitType)
{
	ReadmeContent = InText;
}

FText SGitSourceControlSettings::GetReadmeContent() const
{
	return ReadmeContent;
}

void SGitSourceControlSettings::OnCheckedCreateGitAttributes(ECheckBoxState NewCheckedState)
{
	bAutoCreateGitAttributes = (NewCheckedState == ECheckBoxState::Checked);
}

void SGitSourceControlSettings::OnCheckedInitialCommit(ECheckBoxState NewCheckedState)
{
	bAutoInitialCommit = (NewCheckedState == ECheckBoxState::Checked);
}

void SGitSourceControlSettings::OnInitialCommitMessageCommited(const FText& InText, ETextCommit::Type InCommitType)
{
	InitialCommitMessage = InText;
}

FText SGitSourceControlSettings::GetInitialCommitMessage() const
{
	return InitialCommitMessage;
}

void SGitSourceControlSettings::OnRemoteUrlCommited(const FText& InText, ETextCommit::Type InCommitType)
{
	RemoteUrl = InText;
}

FText SGitSourceControlSettings::GetRemoteUrl() const
{
	return RemoteUrl;
}

#undef LOCTEXT_NAMESPACE
