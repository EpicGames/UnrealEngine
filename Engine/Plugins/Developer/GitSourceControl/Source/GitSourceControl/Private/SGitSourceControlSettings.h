// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "ISourceControlProvider.h"

class SNotificationItem;
namespace ETextCommit { enum Type : int; }

enum class ECheckBoxState : uint8;

class SGitSourceControlSettings : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SGitSourceControlSettings) {}
	
	SLATE_END_ARGS()

public:

	void Construct(const FArguments& InArgs);

	~SGitSourceControlSettings();

private:

	/** Delegates to get Git binary path from/to settings */
	FString GetBinaryPathString() const;
	void OnBinaryPathPicked(const FString & PickedPath) const;

	/** Delegate to get repository root, user name and email from provider */
	FText GetPathToRepositoryRoot() const;
	FText GetUserName() const;
	FText GetUserEmail() const;

	EVisibility CanInitializeGitRepository() const;
	bool CanInitializeGitLfs() const;

	/** Delegate to initialize a new Git repository */
	FReply OnClickedInitializeGitRepository();

	void OnCheckedCreateGitIgnore(ECheckBoxState NewCheckedState);
	bool bAutoCreateGitIgnore;

	/** Delegates to create a README.md file */
	void OnCheckedCreateReadme(ECheckBoxState NewCheckedState);
	bool GetAutoCreateReadme() const;
	bool bAutoCreateReadme;
	void OnReadmeContentCommited(const FText& InText, ETextCommit::Type InCommitType);
	FText GetReadmeContent() const;
	FText ReadmeContent;

	void OnCheckedCreateGitAttributes(ECheckBoxState NewCheckedState);
	bool bAutoCreateGitAttributes;

	void OnCheckedInitialCommit(ECheckBoxState NewCheckedState);
	bool bAutoInitialCommit;
	void OnInitialCommitMessageCommited(const FText& InText, ETextCommit::Type InCommitType);
	FText GetInitialCommitMessage() const;
	FText InitialCommitMessage;

	void OnRemoteUrlCommited(const FText& InText, ETextCommit::Type InCommitType);
	FText GetRemoteUrl() const;
	FText RemoteUrl;

	/** Launch initial asynchronous add and commit operations */
	void LaunchMarkForAddOperation(const TArray<FString>& InFiles);
	void LaunchCheckInOperation();

	/** Delegate called when a source control operation has completed */
	void OnSourceControlOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);

	/** Asynchronous operation progress notifications */
	TWeakPtr<SNotificationItem> OperationInProgressNotification;
	
	void DisplayInProgressNotification(const FSourceControlOperationRef& InOperation);
	void RemoveInProgressNotification();
	void DisplaySuccessNotification(const FSourceControlOperationRef& InOperation);
	void DisplayFailureNotification(const FSourceControlOperationRef& InOperation);
};
